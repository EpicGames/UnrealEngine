// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class EOSShared_WinGDK : EOSShared
{
	public EOSShared_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
