{
	"FileVersion": 3,

	"FriendlyName": "EOS Shared",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Responsible for init/shutdown of the EOSSDK runtime library.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "EOSShared",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Linux",
				"LinuxArm64",
				"Mac",
				"Win64"
			]
		}
	],

	"Plugins": [
		{
			"Name": "SteamShared",
			"Enabled": false,
			"Optional": true
		}
	]
}
