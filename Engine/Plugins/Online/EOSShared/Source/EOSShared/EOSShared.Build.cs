// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.IO;
using UnrealBuildTool;

public class EOSShared : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Engine, "EOSShared")]
	bool bEnableApiVersionWarnings = true;

	protected virtual bool EnableApiVersionWarnings
	{
		get
		{
			ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);
			return bEnableApiVersionWarnings;
		}
	}

	public EOSShared(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		Type = ModuleType.CPlusPlus;

		bRequiresPlatformSDK = true;

		PublicDefinitions.Add("UE_WITH_EOS_SDK_APIVERSION_WARNINGS=" + (EnableApiVersionWarnings ? "1" : "0"));

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"EOSSDK"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
		}
		
		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Slate");
			PrivateIncludePathModuleNames.Add("Engine");
		}

		// On these platforms, the EOS SDK supports a native integration with Steam. 
		bool bWithSteamIntegration = (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64) || Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux;
		PrivateDefinitions.Add("UE_WITH_EOS_STEAM_INTEGRATION=" + (bWithSteamIntegration ? "1" : "0"));
		if (bWithSteamIntegration)
		{
			PrivateIncludePathModuleNames.Add("SteamShared");
		}
	}
}
