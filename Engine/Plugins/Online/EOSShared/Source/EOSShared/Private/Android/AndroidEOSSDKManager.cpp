// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidEOSSDKManager.h"

#if WITH_EOS_SDK
#include "EOSShared.h"
#include "HAL/FileManager.h"
#include "Misc/CoreDelegates.h"

#include "eos_android.h"

FAndroidEOSSDKManager::FAndroidEOSSDKManager()
{
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FAndroidEOSSDKManager::OnApplicationStatusChanged, EOS_EApplicationStatus::EOS_AS_Foreground);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FAndroidEOSSDKManager::OnApplicationStatusChanged, EOS_EApplicationStatus::EOS_AS_BackgroundSuspended);
}

FAndroidEOSSDKManager::~FAndroidEOSSDKManager()
{	
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
}

EOS_EResult FAndroidEOSSDKManager::EOSInitialize(EOS_InitializeOptions& Options)
{
	EOS_Android_InitializeOptions SystemInitializeOptions = { 0 };
	SystemInitializeOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_ANDROID_INITIALIZEOPTIONS_API_LATEST, 2);
	SystemInitializeOptions.Reserved = nullptr;
	SystemInitializeOptions.OptionalInternalDirectory = nullptr;
	SystemInitializeOptions.OptionalExternalDirectory = nullptr;

	Options.SystemInitializeOptions = &SystemInitializeOptions;

	return FEOSSDKManager::EOSInitialize(Options);
}

FString FAndroidEOSSDKManager::GetCacheDirBase() const
{
	FString BaseCacheDirBase = FEOSSDKManager::GetCacheDirBase();
	return IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*BaseCacheDirBase);
};

#endif // WITH_EOS_SDK