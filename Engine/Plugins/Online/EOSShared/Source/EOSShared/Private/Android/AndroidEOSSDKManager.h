// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSSDKManager.h"

class FAndroidEOSSDKManager : public FEOSSDKManager
{
public:
	FAndroidEOSSDKManager();
	~FAndroidEOSSDKManager();

	virtual EOS_EResult EOSInitialize(EOS_InitializeOptions& Options) override;
	virtual FString GetCacheDirBase() const override;
};

using FPlatformEOSSDKManager = FAndroidEOSSDKManager;

#endif // WITH_EOS_SDK