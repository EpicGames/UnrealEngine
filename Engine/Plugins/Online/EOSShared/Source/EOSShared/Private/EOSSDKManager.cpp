// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSSDKManager.h"

#if WITH_EOS_SDK

#include "Algo/AnyOf.h"
#include "Containers/Ticker.h"
#include "HAL/LowLevelMemTracker.h"

#if WITH_ENGINE
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformInput.h"
#include "InputCoreTypes.h"
#endif

#include "Misc/ScopeRWLock.h"
#include "Misc/App.h"
#include "Misc/AsciiSet.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Fork.h"
#include "Misc/NetworkVersion.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "Stats/Stats.h"

#include "CoreGlobals.h"

#include "EOSShared.h"
#include "EOSSharedModule.h"
#include "EOSSharedTypes.h"

#include "eos_auth.h"
#include "eos_connect.h"
#include "eos_friends.h"
#include "eos_init.h"
#include "eos_integratedplatform_types.h"
#include "eos_logging.h"
#include "eos_presence.h"
#include "eos_userinfo.h"
#include "eos_version.h"

#ifndef EOS_TRACE_MALLOC
#define EOS_TRACE_MALLOC 1
#endif

namespace
{
	static void* EOS_MEMORY_CALL EosMalloc(size_t Bytes, size_t Alignment)
	{
		LLM_SCOPE(ELLMTag::RealTimeCommunications);

#if !EOS_TRACE_MALLOC
		CALLSTACK_TRACE_LIMIT_CALLSTACKRESOLVE_SCOPE();
#endif

		return FMemory::Malloc(Bytes, Alignment);
	}

	static void* EOS_MEMORY_CALL EosRealloc(void* Ptr, size_t Bytes, size_t Alignment)
	{
		LLM_SCOPE(ELLMTag::RealTimeCommunications);

#if !EOS_TRACE_MALLOC
		CALLSTACK_TRACE_LIMIT_CALLSTACKRESOLVE_SCOPE();
#endif

		return FMemory::Realloc(Ptr, Bytes, Alignment);
	}

	static void EOS_MEMORY_CALL EosFree(void* Ptr)
	{
		LLM_SCOPE(ELLMTag::RealTimeCommunications);

#if !EOS_TRACE_MALLOC
		CALLSTACK_TRACE_LIMIT_CALLSTACKRESOLVE_SCOPE();
#endif

		FMemory::Free(Ptr);
	}

#if !NO_LOGGING
	void EOS_CALL EOSLogMessageReceived(const EOS_LogMessage* Message)
	{
		FString MessageStr(UTF8_TO_TCHAR(Message->Message));
		MessageStr.TrimStartAndEndInline();
		FString CategoryStr(UTF8_TO_TCHAR(Message->Category));
		CategoryStr.TrimStartAndEndInline();
		
		EOS_ELogLevel EOSLogLevel = Message->Level;
			
		if (FEOSSharedModule* Module = FEOSSharedModule::Get())
		{
			auto MessageProjection = [&MessageStr](const FString& Elem) { return MessageStr.Contains(Elem); };
			auto CategoryProjection = [&CategoryStr](const FString& Elem) { return CategoryStr.Contains(Elem); };

			if (EOSLogLevel < EOS_ELogLevel::EOS_LOG_VeryVerbose
				&& (Algo::AnyOf(Module->GetSuppressedLogStrings_VeryVerbose(), MessageProjection)
				|| Algo::AnyOf(Module->GetSuppressedLogCategories_VeryVerbose(), CategoryProjection)))
			{
				EOSLogLevel = EOS_ELogLevel::EOS_LOG_VeryVerbose;
			}

			if (EOSLogLevel < EOS_ELogLevel::EOS_LOG_Info
				&& (Algo::AnyOf(Module->GetSuppressedLogStrings_Log(), MessageProjection)
				|| Algo::AnyOf(Module->GetSuppressedLogCategories_Log(), CategoryProjection)))
			{
				EOSLogLevel = EOS_ELogLevel::EOS_LOG_Info;
			}
		}

		#define EOSLOG(Level) UE_LOGF(LogEOSSDK, Level, "%ls: %ls", *CategoryStr, *MessageStr)
		switch (EOSLogLevel)
		{
		case EOS_ELogLevel::EOS_LOG_Fatal:			EOSLOG(Fatal); break;
		case EOS_ELogLevel::EOS_LOG_Error:			EOSLOG(Error); break;
		case EOS_ELogLevel::EOS_LOG_Warning:		EOSLOG(Warning); break;
		case EOS_ELogLevel::EOS_LOG_Info:			EOSLOG(Log); break;
		case EOS_ELogLevel::EOS_LOG_Verbose:		EOSLOG(Verbose); break;
		case EOS_ELogLevel::EOS_LOG_VeryVerbose:	EOSLOG(VeryVerbose); break;
		case EOS_ELogLevel::EOS_LOG_Off:
		default:
			// do nothing
			break;
		}
		#undef EOSLOG
	}

	EOS_ELogLevel ConvertLogLevel(ELogVerbosity::Type UELogLevel)
	{
		switch (UELogLevel)
		{
		case ELogVerbosity::NoLogging:		return EOS_ELogLevel::EOS_LOG_Off;
		case ELogVerbosity::Fatal:			return EOS_ELogLevel::EOS_LOG_Fatal;
		case ELogVerbosity::Error:			return EOS_ELogLevel::EOS_LOG_Error;
		case ELogVerbosity::Warning:		return EOS_ELogLevel::EOS_LOG_Warning;
		default:							// Intentional fall through
		case ELogVerbosity::Display:		// Intentional fall through
		case ELogVerbosity::Log:			return EOS_ELogLevel::EOS_LOG_Info;
		case ELogVerbosity::Verbose:		return EOS_ELogLevel::EOS_LOG_Verbose;
		case ELogVerbosity::VeryVerbose:	return EOS_ELogLevel::EOS_LOG_VeryVerbose;
		}
	}
#endif // !NO_LOGGING
}

#if EOSSDK_RUNTIME_LOAD_REQUIRED
void FEOSSDKManager::LoadSDKHandle()
{
	check(SDKHandle == nullptr);

	auto AttemptLoadDll = [](const FString& BinaryPath) -> void*
	{
		UE_LOGF(LogEOSShared, Verbose, "Attempting to load \"%ls\"", *BinaryPath);
		void* Result = FPlatformProcess::GetDllHandle(*BinaryPath);
		UE_CLOGF(Result, LogEOSShared, Log, "Loaded \"%ls\"", *BinaryPath);
		UE_CLOGF(!Result, LogEOSShared, Verbose, "Failed to load \"%ls\"", *BinaryPath);
		return Result;
	};

#if !UE_BUILD_SHIPPING
	FString CommandLineBinary;
	if (FParse::Value(FCommandLine::Get(), TEXT("eossdkbinary="), CommandLineBinary))
	{
		SDKHandle = AttemptLoadDll(CommandLineBinary);
	}
	else
#endif // !UE_BUILD_SHIPPING
	{
		const FString RuntimeLibraryName = OnRequestRuntimeLibraryName.IsBound()
			? OnRequestRuntimeLibraryName.Execute()
		: TEXT(EOSSDK_RUNTIME_LIBRARY_NAME);

		const FString ProjectBinaryPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory(), *RuntimeLibraryName));
		if (FPaths::FileExists(ProjectBinaryPath))
		{
			SDKHandle = AttemptLoadDll(ProjectBinaryPath);
		}

		if (!SDKHandle)
		{
			const FString EngineBinaryPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory(), *RuntimeLibraryName));
			if (FPaths::FileExists(EngineBinaryPath))
			{
				SDKHandle = AttemptLoadDll(EngineBinaryPath);
			}
		}

		if (!SDKHandle)
		{
			SDKHandle = AttemptLoadDll(RuntimeLibraryName);
		}
	}

	if (!SDKHandle)
	{
		bool bDllLoadFailureIsFatal = false;
		GConfig->GetBool(TEXT("EOSSDK"), TEXT("bDllLoadFailureIsFatal"), bDllLoadFailureIsFatal, GEngineIni);
		if (bDllLoadFailureIsFatal)
		{
			FPlatformMisc::MessageBoxExt(
				EAppMsgType::Ok,
				*NSLOCTEXT("EOSShared", "DllLoadFail", "Failed to load EOSSDK. Please verify your installation. Exiting...").ToString(),
				TEXT("Error")
			);
			UE_LOGF(LogEOSShared, Fatal, "%s Failed to load EOSSDK binary", __FUNCTION__);
		}
	}
}
#endif

FEOSSDKManager::FEOSSDKManager()
{
}

FEOSSDKManager::~FEOSSDKManager()
{
#if EOSSDK_RUNTIME_LOAD_REQUIRED
	if (SDKHandle != nullptr)
	{
		FPlatformProcess::FreeDllHandle(SDKHandle);
	}
#endif
}

EOS_EResult FEOSSDKManager::Initialize()
{
	if (FForkProcessHelper::IsForkRequested() && !FForkProcessHelper::IsForkedChildProcess())
	{
		UE_LOGF(LogEOSShared, Error, "%s Initialize failed, pre-fork", __FUNCTION__);
		return EOS_EResult::EOS_InvalidState;
	}

#if EOSSDK_RUNTIME_LOAD_REQUIRED
	if (SDKHandle == nullptr)
	{
		LLM_SCOPE(ELLMTag::RealTimeCommunications);
		LoadSDKHandle();
	}
	if (SDKHandle == nullptr)
	{
		UE_LOGF(LogEOSShared, Log, "%s failed, SDKHandle=nullptr", __FUNCTION__);
		return EOS_EResult::EOS_InvalidState;
	}
#endif

	if (IsInitialized())
	{
		return EOS_EResult::EOS_Success;
	}
	else
	{
		UE_LOGF(LogEOSShared, Log, "%s Initializing EOSSDK Version:%ls", __FUNCTION__, UTF8_TO_TCHAR(EOS_GetVersion()));

		FString ProductName = GetProductName();
		constexpr FAsciiSet ValidProductNameChars("ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789._ !?&()+-:");
		if (!FAsciiSet::HasOnly(*ProductName, ValidProductNameChars))
		{
			UE_LOGF(LogEOSShared, Warning, "ProductName=%ls contains invalid characters, please set a valid one", *ProductName);

			while (TCHAR* BadChar = const_cast<TCHAR*>(FAsciiSet::Skip(*ProductName, ValidProductNameChars)))
			{
				if (*BadChar == TCHAR('\0'))
				{
					break;
				}
				*BadChar = TCHAR(' ');
			}
		}
		const FTCHARToUTF8 ProductNameUtf8(*ProductName);
		const FTCHARToUTF8 ProductVersion(*GetProductVersion());

		EOS_InitializeOptions InitializeOptions = {};
		InitializeOptions.ApiVersion = 4;
		UE_EOS_CHECK_API_MISMATCH(EOS_INITIALIZE_API_LATEST, 5);
		InitializeOptions.AllocateMemoryFunction = &EosMalloc;
		InitializeOptions.ReallocateMemoryFunction = &EosRealloc;
		InitializeOptions.ReleaseMemoryFunction = &EosFree;
		InitializeOptions.ProductName = ProductNameUtf8.Get();
		InitializeOptions.ProductVersion = ProductVersion.Length() > 0 ? ProductVersion.Get() : nullptr;
		InitializeOptions.Reserved = nullptr;
		InitializeOptions.SystemInitializeOptions = nullptr;
		InitializeOptions.OverrideThreadAffinity = nullptr;

		EOS_EResult EosResult = EOSInitialize(InitializeOptions);

		if (EosResult == EOS_EResult::EOS_Success)
		{
			bInitialized = true;

			FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FEOSSDKManager::OnConfigSectionsChanged);
			LoadConfig();

#if !NO_LOGGING
			FCoreDelegates::OnLogVerbosityChanged.AddRaw(this, &FEOSSDKManager::OnLogVerbosityChanged);

			EosResult = EOS_Logging_SetCallback(&EOSLogMessageReceived);
			if (EosResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogEOSShared, Warning, "%s EOS_Logging_SetCallback failed error:%ls", __FUNCTION__, *LexToString(EosResult));
			}

			EosResult = EOS_Logging_SetLogLevel(EOS_ELogCategory::EOS_LC_ALL_CATEGORIES, ConvertLogLevel(LogEOSSDK.GetVerbosity()));
			if (EosResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogEOSShared, Warning, "%s EOS_Logging_SetLogLevel failed Verbosity=%ls error=[%ls]", __FUNCTION__, ToString(LogEOSSDK.GetVerbosity()), *LexToString(EosResult));
			}
#endif // !NO_LOGGING

			FCoreDelegates::OnNetworkConnectionStatusChanged.AddRaw(this, &FEOSSDKManager::OnNetworkConnectionStatusChanged);
		}
		else
		{
			UE_LOGF(LogEOSShared, Warning, "%s EOS_Initialize failed error:%ls", __FUNCTION__, *LexToString(EosResult));
		}

		OnPostInitializeSDK.Broadcast(EosResult);
		return EosResult;
	}
}

const FEOSSDKPlatformConfig* FEOSSDKManager::GetPlatformConfig(const FString& PlatformConfigName, bool bLoadIfMissing)
{
	if (PlatformConfigName.IsEmpty())
	{
		UE_LOGF(LogEOSShared, Warning, "%s PlatformConfigName empty", __FUNCTION__);
		return nullptr;
	}

	FEOSSDKPlatformConfig* PlatformConfig = PlatformConfigs.Find(PlatformConfigName);
	if (PlatformConfig || !bLoadIfMissing)
	{
		return PlatformConfig;
	}

	const FString SectionName(TEXT("EOSSDK.Platform.") + PlatformConfigName);
	if (!GConfig->DoesSectionExist(*SectionName, GEngineIni))
	{
		UE_LOGF(LogEOSShared, Warning, "%s PlatformConfigName \"%ls\" not found", __FUNCTION__, *PlatformConfigName);
		return nullptr;
	}

	PlatformConfig = &PlatformConfigs.Emplace(PlatformConfigName);

	PlatformConfig->Name = PlatformConfigName;
	GConfig->GetString(*SectionName, TEXT("ProductId"), PlatformConfig->ProductId, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("SandboxId"), PlatformConfig->SandboxId, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("ClientId"), PlatformConfig->ClientId, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("ClientSecret"), PlatformConfig->ClientSecret, GEngineIni);
	if (GConfig->GetString(*SectionName, TEXT("EncryptionKey"), PlatformConfig->EncryptionKey, GEngineIni))
	{
		// EncryptionKey gets removed from packaged builds due to IniKeyDenylist=EncryptionKey entry in BaseGame.ini.
		// Normally we could just add a remap in ConfigRedirects.ini but the section name varies with the PlatformConfigName.
		UE_LOGF(LogEOSShared, Warning, "%s Config section \"EOSSDK.Platform.%ls\" contains deprecated key EncryptionKey, please migrate to ClientEncryptionKey.", __FUNCTION__, *PlatformConfigName);
	}
	GConfig->GetString(*SectionName, TEXT("ClientEncryptionKey"), PlatformConfig->EncryptionKey, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("RelyingPartyURI"), PlatformConfig->RelyingPartyURI, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("OverrideCountryCode"), PlatformConfig->OverrideCountryCode, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("OverrideLocaleCode"), PlatformConfig->OverrideLocaleCode, GEngineIni);
	GConfig->GetString(*SectionName, TEXT("DeploymentId"), PlatformConfig->DeploymentId, GEngineIni);

	if (GConfig->GetString(*SectionName, TEXT("CacheBaseSubdirectory"), PlatformConfig->CacheDirectory, GEngineIni))
	{
		const FString CacheDirBase = GetCacheDirBase();
		PlatformConfig->CacheDirectory = CacheDirBase.IsEmpty() ? FString() : CacheDirBase / PlatformConfig->CacheDirectory;
	}
	GConfig->GetString(*SectionName, TEXT("CacheDirectory"), PlatformConfig->CacheDirectory, GEngineIni);

	bool bCheckRuntimeType = false;
	GConfig->GetBool(*SectionName, TEXT("bCheckRuntimeType"), bCheckRuntimeType, GEngineIni);
	if (bCheckRuntimeType)
	{
		PlatformConfig->bIsServer = IsRunningDedicatedServer();
		PlatformConfig->bLoadingInEditor = !IsRunningGame() && !IsRunningDedicatedServer();

		if (PlatformConfig->bIsServer || PlatformConfig->bLoadingInEditor)
		{
			// Don't attempt to load overlay for servers or editors.
			PlatformConfig->bDisableOverlay = true;
			PlatformConfig->bDisableSocialOverlay = true;
		}
		else
		{
			// Overlay is on by default, enable additional overlay options.
			PlatformConfig->bWindowsEnableOverlayD3D9 = true;
			PlatformConfig->bWindowsEnableOverlayD3D10 = true;
			PlatformConfig->bWindowsEnableOverlayOpenGL = true;
		}
	}

	GConfig->GetBool(*SectionName, TEXT("bIsServer"), PlatformConfig->bIsServer, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bLoadingInEditor"), PlatformConfig->bLoadingInEditor, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bDisableOverlay"), PlatformConfig->bDisableOverlay, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bDisableSocialOverlay"), PlatformConfig->bDisableSocialOverlay, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bWindowsEnableOverlayD3D9"), PlatformConfig->bWindowsEnableOverlayD3D9, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bWindowsEnableOverlayD3D10"), PlatformConfig->bWindowsEnableOverlayD3D10, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bWindowsEnableOverlayOpenGL"), PlatformConfig->bWindowsEnableOverlayOpenGL, GEngineIni);
	GConfig->GetBool(*SectionName, TEXT("bEnableRTC"), PlatformConfig->bEnableRTC, GEngineIni);
	GConfig->GetInt(*SectionName, TEXT("TickBudgetInMilliseconds"), PlatformConfig->TickBudgetInMilliseconds, GEngineIni);
	GConfig->GetArray(*SectionName, TEXT("OptionalConfig"), PlatformConfig->OptionalConfig, GEngineIni);

	// After we have loaded the platform config, we'll check to see if there is any command-line overrides present

	FString SandboxIdOverride;
	// Get the -epicsandboxid argument. This generally comes from EGS.
	bool bHasSandboxIdOverride = FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxId="), SandboxIdOverride);
	// Prefer -EpicSandboxIdOverride over previous.
	bHasSandboxIdOverride |= FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxIdOverride="), SandboxIdOverride);

	if (bHasSandboxIdOverride)
	{
		PlatformConfig->SandboxId = SandboxIdOverride;
	}

	FString DeploymentIdOverride;
	// Get the -epicdeploymentid argument. This generally comes from EGS.
	bool bHasDeploymentIdOverride = FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentId="), DeploymentIdOverride);
	// Prefer -EpicDeploymentIdOverride over previous.
	bHasDeploymentIdOverride |= FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentIdOverride="), DeploymentIdOverride);

	if (bHasDeploymentIdOverride)
	{
		PlatformConfig->DeploymentId = DeploymentIdOverride;
	}

	UE_LOGF(LogEOSShared, Verbose, "%s Loaded platform config: %ls", __FUNCTION__, *PlatformConfigName);
	return PlatformConfig;
}

bool FEOSSDKManager::AddPlatformConfig(const FEOSSDKPlatformConfig& PlatformConfig, bool bOverwriteExistingConfig)
{
	if (PlatformConfig.Name.IsEmpty())
	{
		UE_LOGF(LogEOSShared, Warning, "%s Platform name can't be empty", __FUNCTION__);
		return false;
	}

	if (PlatformConfigs.Find(PlatformConfig.Name) && !bOverwriteExistingConfig)
	{
		UE_LOGF(LogEOSShared, Warning, "%s Platform config already exists: %ls", __FUNCTION__, *PlatformConfig.Name);
		return false;
	}

	PlatformConfigs.Emplace(PlatformConfig.Name, PlatformConfig);
	UE_LOGF(LogEOSShared, Verbose, "%s Added platform config: %ls", __FUNCTION__, *PlatformConfig.Name);
	return true;
}

const FString& FEOSSDKManager::GetDefaultPlatformConfigName()
{
	if (DefaultPlatformConfigName.IsEmpty())
	{
		FString PlatformConfigName;
		if (GConfig->GetString(TEXT("EOSSDK"), TEXT("DefaultPlatformConfigName"), PlatformConfigName, GEngineIni))
		{
			SetDefaultPlatformConfigName(PlatformConfigName);
		}
	}

	return DefaultPlatformConfigName;
}

void FEOSSDKManager::SetDefaultPlatformConfigName(const FString& PlatformConfigName)
{
	if (DefaultPlatformConfigName != PlatformConfigName)
	{
		UE_LOGF(LogEOSShared, Verbose, "%s Default platform name changed: New=%ls Old=%ls", __FUNCTION__, *PlatformConfigName, *DefaultPlatformConfigName);
		OnDefaultPlatformConfigNameChanged.Broadcast(PlatformConfigName, DefaultPlatformConfigName);
		DefaultPlatformConfigName = PlatformConfigName;
	}
}

EOS_HIntegratedPlatformOptionsContainer FEOSSDKManager::CreateIntegratedPlatformOptionsContainer()
{
	EOS_HIntegratedPlatformOptionsContainer Result;

	EOS_IntegratedPlatform_CreateIntegratedPlatformOptionsContainerOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_INTEGRATEDPLATFORM_CREATEINTEGRATEDPLATFORMOPTIONSCONTAINER_API_LATEST, 1);

	EOS_EResult CreationResult = EOS_IntegratedPlatform_CreateIntegratedPlatformOptionsContainer(&Options, &Result);
	if (CreationResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogEOSShared, Warning, "%s EOS_IntegratedPlatform_CreateIntegratedPlatformOptionsContainer Result=[%ls]", __FUNCTION__, *LexToString(CreationResult));
	}

	return Result;
}

const void* FEOSSDKManager::GetIntegratedPlatformOptions()
{
	return nullptr;
}

EOS_IntegratedPlatformType FEOSSDKManager::GetIntegratedPlatformType()
{
	return EOS_IPT_Unknown;
}

void FEOSSDKManager::ApplyIntegratedPlatformOptions(EOS_HIntegratedPlatformOptionsContainer& Container)
{
	if (IsRunningCommandlet())
	{
		UE_LOGF(LogEOSShared, Verbose, "%s Method not supported when running Commandlet", __FUNCTION__);
		Container = nullptr;
		return;
	}

	if (bEnablePlatformIntegration)
	{
		Container = CreateIntegratedPlatformOptionsContainer();

		if (Container != nullptr)
		{
			//UE does not support EOS_IPMF_LibraryManagedBySDK due to functionality overlap
			EOS_IntegratedPlatform_Options PlatformOptions = {};
			PlatformOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_INTEGRATEDPLATFORM_OPTIONS_API_LATEST, 1);
			PlatformOptions.Type = GetIntegratedPlatformType();
			PlatformOptions.Flags = IntegratedPlatformManagementFlags;
			PlatformOptions.InitOptions = GetIntegratedPlatformOptions();

			EOS_IntegratedPlatformOptionsContainer_AddOptions AddOptions = {};
			AddOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_INTEGRATEDPLATFORMOPTIONSCONTAINER_ADD_API_LATEST, 1);
			AddOptions.Options = &PlatformOptions;

			EOS_EResult Result = EOS_IntegratedPlatformOptionsContainer_Add(Container, &AddOptions);
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogEOSShared, Warning, "%s EOS_IntegratedPlatformOptionsContainer_Add Result=[%ls]", __FUNCTION__, *LexToString(Result));
			}
		}
	}
	else
	{
		Container = nullptr;
	}
}

void FEOSSDKManager::ApplySystemSpecificOptions(const void*& SystemSpecificOptions)
{
	SystemSpecificOptions = nullptr;
}

IEOSPlatformHandlePtr FEOSSDKManager::CreatePlatform(const FString& PlatformConfigName, FName InstanceName)
{
	if (PlatformConfigName.IsEmpty())
	{
		UE_LOGF(LogEOSShared, Warning, "%s Platform name can't be empty", __FUNCTION__);
		return IEOSPlatformHandlePtr();
	}

	const FEOSSDKPlatformConfig* const PlatformConfig = GetPlatformConfig(PlatformConfigName, true);
	if (!PlatformConfig)
	{
		return IEOSPlatformHandlePtr();
	}

	if (PlatformConfig->ProductId.IsEmpty() ||
		PlatformConfig->SandboxId.IsEmpty() ||
		PlatformConfig->DeploymentId.IsEmpty() ||
		PlatformConfig->ClientId.IsEmpty() ||
		PlatformConfig->ClientSecret.IsEmpty())
	{
		UE_LOGF(LogEOSShared, Warning, "%s Platform config missing required options", __FUNCTION__);
		return IEOSPlatformHandlePtr();
	}

	TMap<FName, IEOSPlatformHandleWeakPtr>* PlatformMap = PlatformHandles.Find(PlatformConfigName);
	if (PlatformMap)
	{
		IEOSPlatformHandleWeakPtr* WeakPlatformHandle = PlatformMap->Find(InstanceName);
		if (WeakPlatformHandle)
		{
			if (IEOSPlatformHandlePtr Pinned = WeakPlatformHandle->Pin())
			{
				UE_LOGF(LogEOSShared, Verbose, "%s Found existing platform handle: PlatformConfigName=%ls InstanceName=%ls", __FUNCTION__, *PlatformConfigName, *InstanceName.ToString());
				return Pinned;
			}

			UE_LOGF(LogEOSShared, Verbose, "%s Removing stale platform handle pointer: PlatformConfigName=%ls InstanceName=%ls", __FUNCTION__, *PlatformConfigName, *InstanceName.ToString());
			PlatformMap->Remove(InstanceName);
		}
	}
	else
	{
		PlatformMap = &PlatformHandles.Emplace(PlatformConfigName);
	}

	const FTCHARToUTF8 Utf8ProductId(*PlatformConfig->ProductId);
	const FTCHARToUTF8 Utf8SandboxId(*PlatformConfig->SandboxId);
	const FTCHARToUTF8 Utf8ClientId(*PlatformConfig->ClientId);
	const FTCHARToUTF8 Utf8ClientSecret(*PlatformConfig->ClientSecret);
	const FTCHARToUTF8 Utf8EncryptionKey(*PlatformConfig->EncryptionKey);
	const FTCHARToUTF8 Utf8OverrideCountryCode(*PlatformConfig->OverrideCountryCode);
	const FTCHARToUTF8 Utf8OverrideLocaleCode(*PlatformConfig->OverrideLocaleCode);
	const FTCHARToUTF8 Utf8DeploymentId(*PlatformConfig->DeploymentId);
	const FTCHARToUTF8 Utf8CacheDirectory(*PlatformConfig->CacheDirectory);

	EOS_Platform_Options PlatformOptions = {};
	PlatformOptions.ApiVersion = 13;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_OPTIONS_API_LATEST, 15);
	PlatformOptions.Reserved = nullptr;
	PlatformOptions.ProductId = Utf8ProductId.Length() ? Utf8ProductId.Get() : nullptr;
	PlatformOptions.SandboxId = Utf8SandboxId.Length() ? Utf8SandboxId.Get() : nullptr;
	PlatformOptions.ClientCredentials.ClientId = Utf8ClientId.Length() ? Utf8ClientId.Get() : nullptr;
	PlatformOptions.ClientCredentials.ClientSecret = Utf8ClientSecret.Length() ? Utf8ClientSecret.Get() : nullptr;
	PlatformOptions.bIsServer = PlatformConfig->bIsServer;
	PlatformOptions.EncryptionKey = Utf8EncryptionKey.Length() ? Utf8EncryptionKey.Get() : nullptr;
	PlatformOptions.OverrideCountryCode = Utf8OverrideCountryCode.Length() ? Utf8OverrideCountryCode.Get() : nullptr;
	PlatformOptions.OverrideLocaleCode = Utf8OverrideLocaleCode.Length() ? Utf8OverrideLocaleCode.Get() : nullptr;
	PlatformOptions.DeploymentId = Utf8DeploymentId.Length() ? Utf8DeploymentId.Get() : nullptr;

	PlatformOptions.Flags = 0;
	if (PlatformConfig->bLoadingInEditor) PlatformOptions.Flags |= EOS_PF_LOADING_IN_EDITOR;
	if (PlatformConfig->bDisableOverlay) PlatformOptions.Flags |= EOS_PF_DISABLE_OVERLAY;
	if (PlatformConfig->bDisableSocialOverlay) PlatformOptions.Flags |= EOS_PF_DISABLE_SOCIAL_OVERLAY;

	if (FPlatformMisc::IsCacheStorageAvailable())
	{
		PlatformOptions.CacheDirectory = Utf8CacheDirectory.Length() ? Utf8CacheDirectory.Get() : nullptr;
	}
	else
	{
		PlatformOptions.CacheDirectory = nullptr;
	}

	PlatformOptions.TickBudgetInMilliseconds = PlatformConfig->TickBudgetInMilliseconds;
	PlatformOptions.TaskNetworkTimeoutSeconds = nullptr;

	EOS_Platform_RTCOptions PlatformRTCOptions = {};
	PlatformRTCOptions.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_RTCOPTIONS_API_LATEST, 3);
	PlatformRTCOptions.PlatformSpecificOptions = nullptr;
	PlatformRTCOptions.BackgroundMode = PlatformConfig->RTCBackgroundMode;
	PlatformRTCOptions.Reserved = nullptr;

	PlatformOptions.RTCOptions = PlatformConfig->bEnableRTC ? &PlatformRTCOptions : nullptr;

	IEOSPlatformHandlePtr PlatformHandle = CreatePlatform(*PlatformConfig, PlatformOptions);
	if (PlatformHandle.IsValid())
	{
		UE_LOGF(LogEOSShared, Verbose, "%s Created platform handle: PlatformConfigName=%ls InstanceName=%ls", __FUNCTION__, *PlatformConfigName, *InstanceName.ToString());
		PlatformMap->Emplace(InstanceName, PlatformHandle);
	}

	return PlatformHandle;
}

IEOSPlatformHandlePtr FEOSSDKManager::CreatePlatform(const FEOSSDKPlatformConfig& PlatformConfig, EOS_Platform_Options& PlatformOptions)
{
	OnPreCreateNamedPlatform.Broadcast(PlatformConfig, PlatformOptions);
	IEOSPlatformHandlePtr Result = CreatePlatform(PlatformOptions);
	if (Result)
	{
		static_cast<FEOSPlatformHandle&>(*Result.Get()).ConfigName = PlatformConfig.Name;
	}

	return Result;
}

IEOSPlatformHandlePtr FEOSSDKManager::CreatePlatform(EOS_Platform_Options& PlatformOptions)
{
	check(IsInGameThread());

	IEOSPlatformHandlePtr SharedPlatform;

	if (IsInitialized())
	{
		ApplySystemSpecificOptions(PlatformOptions.SystemSpecificOptions);
		ApplyIntegratedPlatformOptions(PlatformOptions.IntegratedPlatformOptionsContainerHandle);

		OnPreCreatePlatform.Broadcast(PlatformOptions);

		ApplyOverlayPlatformOptions(PlatformOptions);

		const EOS_HPlatform PlatformHandle = EOS_Platform_Create(&PlatformOptions);
		if (PlatformHandle)
		{
			EOS_IntegratedPlatformOptionsContainer_Release(PlatformOptions.IntegratedPlatformOptionsContainerHandle);

			SharedPlatform = MakeShared<FEOSPlatformHandle, ESPMode::ThreadSafe>(*this, PlatformHandle);
			{
				FRWScopeLock ScopeLock(ActivePlatformsCS, SLT_Write);
				ActivePlatforms.Emplace(PlatformHandle, SharedPlatform);
			}

			SetupTicker();

			EOS_Platform_SetApplicationStatus(PlatformHandle, CachedApplicationStatus);
			EOS_Platform_SetNetworkStatus(PlatformHandle, ConvertNetworkStatus(FPlatformMisc::GetNetworkConnectionStatus()));
			
			SetInvokeOverlayButton(PlatformHandle);
			RegisterDisplaySettingsUpdatedCallback(PlatformHandle);

			// Tick the platform once to work around EOSSDK error logging that occurs if you create then immediately destroy a platform.
			SharedPlatform->Tick();

			OnPlatformCreated.Broadcast(SharedPlatform);
		}
		else
		{
			UE_LOGF(LogEOSShared, Warning, "%s failed, EosPlatformHandle=nullptr", __FUNCTION__);
		}
	}
	else
	{
		UE_LOGF(LogEOSShared, Warning, "%s failed, SDK not initialized", __FUNCTION__);
	}

	return SharedPlatform;
}

TArray<IEOSPlatformHandlePtr> FEOSSDKManager::GetActivePlatforms()
{
	FRWScopeLock ScopeLock(ActivePlatformsCS, SLT_ReadOnly);

	TArray<IEOSPlatformHandlePtr> Result;

	for (const TPair<EOS_HPlatform, IEOSPlatformHandleWeakPtr>& Entry : ActivePlatforms)
	{
		if (!ReleasedPlatforms.Contains(Entry.Key))
		{
			if (IEOSPlatformHandlePtr SharedPtr = Entry.Value.Pin())
			{
				Result.Add(SharedPtr);
			}
		}
	}

	return Result;
}

void FEOSSDKManager::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == GEngineIni && SectionNames.Contains(TEXT("EOSSDK")))
	{
		LoadConfig();
	}
}

void FEOSSDKManager::LoadConfig()
{
	const TCHAR* SectionName = TEXT("EOSSDK");

	ConfigTickIntervalSeconds = 0.f;
	GConfig->GetDouble(SectionName, TEXT("TickIntervalSeconds"), ConfigTickIntervalSeconds, GEngineIni);

	bEnablePlatformIntegration = false;
	GConfig->GetBool(SectionName, TEXT("bEnablePlatformIntegration"), bEnablePlatformIntegration, GEngineIni);

	// This used to be part of bEnablePlatformIntegration, so to maintain backwards compatibility, default to that
	bEnableOverlayIntegration = bEnablePlatformIntegration;
	GConfig->GetBool(SectionName, TEXT("bEnableOverlayIntegration"), bEnableOverlayIntegration, GEngineIni);

	InvokeOverlayButtonCombination = EOS_UI_EInputStateButtonFlags::EOS_UISBF_Special_Left;
	FString ButtonCombinationStr;
	GConfig->GetString(SectionName, TEXT("InvokeOverlayButtonCombination"), ButtonCombinationStr, GEngineIni);
	if (!ButtonCombinationStr.IsEmpty())
	{
		EOS_UI_EInputStateButtonFlags ButtonCombination;
		if (LexFromString(ButtonCombination, *ButtonCombinationStr))
		{
			InvokeOverlayButtonCombination = ButtonCombination;
		}
	}

	TArray<FString> ManagementFlags;
	if (GConfig->GetArray(SectionName, TEXT("IntegratedPlatformManagementFlags"), ManagementFlags, GEngineIni))
	{
		IntegratedPlatformManagementFlags = {};
		for (const FString& ManagementFlagStr : ManagementFlags)
		{
			EOS_EIntegratedPlatformManagementFlags NewManagementFlag = {};
			if (!LexFromString(NewManagementFlag, *ManagementFlagStr))
			{
				UE_LOGF(LogEOSShared, Verbose, "%s unknown EOS_EIntegratedPlatformManagementFlags \"%ls\"", __FUNCTION__, *ManagementFlagStr);
			}
			
			IntegratedPlatformManagementFlags |= NewManagementFlag;
		}
	}

	SetupTicker();
}

void FEOSSDKManager::SetupTicker()
{
	check(IsInGameThread());

	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}

	int NumActivePlatforms = ActivePlatforms.Num();
	if (NumActivePlatforms > 0)
	{
		const bool bIsFastTicking = FastTickLock.IsValid();
		const double TickIntervalSeconds = ConfigTickIntervalSeconds > SMALL_NUMBER && !bIsFastTicking ? ConfigTickIntervalSeconds / NumActivePlatforms : 0.f;
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FEOSSDKManager::Tick), TickIntervalSeconds);
	}
}

#if WITH_ENGINE
void FEOSSDKManager::OnBackBufferReady_RenderThread(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
{
	UE_CALL_ONCE([]() {	UE_LOGF(LogEOSShared, VeryVerbose, "%s unimplemented on this platform", __FUNCTION__) });
}
#endif

void FEOSSDKManager::CallUIPrePresent(const EOS_UI_PrePresentOptions& Options)
{
	// This call only returns valid platforms, so we can skip validity checks
	static TMap<EOS_HUI, EOS_EResult> LastResults;
	TArray<IEOSPlatformHandlePtr> ActivePlatformsChecked = GetActivePlatforms();
	for (const IEOSPlatformHandlePtr& ActivePlatform : ActivePlatformsChecked)
	{
		if (EOS_HUI UIHandle = EOS_Platform_GetUIInterface(*ActivePlatform))
		{
			EOS_EResult Result = EOS_UI_PrePresent(UIHandle, &Options);
			EOS_EResult& LastResult = LastResults.FindOrAdd(UIHandle);
			if (LastResult != Result)
			{
				LastResult = Result;
				if (Result == EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogEOSShared, Verbose, "%s EOS_UI_PrePresent is succeeding again.", __FUNCTION__);
				}
				else
				{
					UE_LOGF(LogEOSShared, Verbose, "%s EOS_UI_PrePresent failed with error: %ls", __FUNCTION__, *LexToString(Result));
				}
			}
		}
	}
}

#if WITH_ENGINE
bool FEOSSDKManager::IsRenderReady()
{
	if (bEnableOverlayIntegration)
	{
		if (bRenderReady)
		{
			return true;
		}

		if (!FSlateApplication::IsInitialized())
		{
			return false;
		}

		FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
		if (!Renderer)
		{
			return false;
		}

		Renderer->OnBackBufferReadyToPresent().AddRaw(this, &FEOSSDKManager::OnBackBufferReady_RenderThread);
		bRenderReady = true;
		return true;
	}
	else
	{
		return false;
	}
}
#endif

void FEOSSDKManager::SetInvokeOverlayButton(const EOS_HPlatform PlatformHandle)
{
	if (bEnableOverlayIntegration)
	{
		if (EOS_HUI UIHandle = EOS_Platform_GetUIInterface(PlatformHandle))
		{
			EOS_UI_SetToggleFriendsButtonOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_UI_SETTOGGLEFRIENDSBUTTON_API_LATEST, 1);
			Options.ButtonCombination = InvokeOverlayButtonCombination;

			const EOS_EResult Result = EOS_UI_SetToggleFriendsButton(UIHandle, &Options);
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogEOSShared, Verbose, "%s EOS_UI_SetToggleFriendsButton Result=[%ls]", __FUNCTION__, *LexToString(Result));
			}
		}
	}
}

void FEOSSDKManager::OnDisplaySettingsUpdated(const EOS_UI_OnDisplaySettingsUpdatedCallbackInfo* Data)
{
	static TSharedPtr<IEOSFastTickLock> OverlayFastTickLock;
	if (Data->bIsVisible && !OverlayFastTickLock.IsValid())
	{
		OverlayFastTickLock = reinterpret_cast<FEOSSDKManager*>(Data->ClientData)->GetFastTickLock();
	}
	else if (!Data->bIsVisible && OverlayFastTickLock.IsValid())
	{
		OverlayFastTickLock.Reset();
	}
}

void FEOSSDKManager::RegisterDisplaySettingsUpdatedCallback(const EOS_HPlatform PlatformHandle)
{
	if (bEnableOverlayIntegration)
	{
		if (EOS_HUI UIHandle = EOS_Platform_GetUIInterface(PlatformHandle))
		{
			EOS_UI_AddNotifyDisplaySettingsUpdatedOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_UI_ADDNOTIFYDISPLAYSETTINGSUPDATED_API_LATEST, 1);

			EOS_NotificationId DisplaySettingsUpdatedId = EOS_UI_AddNotifyDisplaySettingsUpdated(UIHandle, &Options, this, &FEOSSDKManager::OnDisplaySettingsUpdated);
		}
	}
}

void FEOSSDKManager::ApplyOverlayPlatformOptions(EOS_Platform_Options& PlatformOptions)
{
	// On consoles, if we're only using the overlay for the AP login flow we can enable auto loading/unloading
	// so that we only load the KITT DLLs when we open a browser and we unload them after closing the browser
	if (PlatformOptions.Flags & EOS_PF_DISABLE_SOCIAL_OVERLAY)
	{
		PlatformOptions.Flags |= EOS_PF_CONSOLE_ENABLE_OVERLAY_AUTOMATIC_UNLOADING;
	}
}

bool FEOSSDKManager::Tick(float)
{
	check(IsInGameThread());

#if WITH_ENGINE
	IsRenderReady();
#endif

	ReleaseReleasedPlatforms();

	if (ActivePlatforms.Num())
	{
		TArray<EOS_HPlatform> PlatformsToTick;

		TArray<EOS_HPlatform> ActivePlatformHandles;
		ActivePlatforms.GenerateKeyArray(ActivePlatformHandles);

		if (ConfigTickIntervalSeconds > SMALL_NUMBER)
		{
			PlatformTickIdx = (PlatformTickIdx + 1) % ActivePlatformHandles.Num();
			PlatformsToTick.Emplace(ActivePlatformHandles[PlatformTickIdx]);
		}
		else
		{
			PlatformsToTick = ActivePlatformHandles;
		}

		for (EOS_HPlatform PlatformHandle : PlatformsToTick)
		{
			LLM_SCOPE(ELLMTag::RealTimeCommunications); // TODO should really be ELLMTag::EOSSDK
			QUICK_SCOPE_CYCLE_COUNTER(FEOSSDKManager_Tick);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSSDK);

			EOS_Platform_Tick(PlatformHandle);
		}
	}

	return true;
}

EOS_ENetworkStatus FEOSSDKManager::ConvertNetworkStatus(ENetworkConnectionStatus Status)
{
	switch (Status)
	{
	case ENetworkConnectionStatus::Unknown:			return EOS_ENetworkStatus::EOS_NS_Online;
	case ENetworkConnectionStatus::Disabled:		return EOS_ENetworkStatus::EOS_NS_Disabled;
	case ENetworkConnectionStatus::Local:			return EOS_ENetworkStatus::EOS_NS_Offline;
	case ENetworkConnectionStatus::Connected:		return EOS_ENetworkStatus::EOS_NS_Online;
	}

	checkNoEntry();
	return EOS_ENetworkStatus::EOS_NS_Disabled;
}

void FEOSSDKManager::OnNetworkConnectionStatusChanged(ENetworkConnectionStatus LastConnectionState, ENetworkConnectionStatus ConnectionState)
{
	check(IsInGameThread());

	const EOS_ENetworkStatus OldNetworkStatus = ConvertNetworkStatus(LastConnectionState);
	const EOS_ENetworkStatus NewNetworkStatus = ConvertNetworkStatus(ConnectionState);

	UE_LOGF(LogEOSShared, Log, "%s [%ls] -> [%ls]", __FUNCTION__, LexToString(OldNetworkStatus), LexToString(NewNetworkStatus));

	for (const TPair<EOS_HPlatform, IEOSPlatformHandleWeakPtr>& Entry : ActivePlatforms)
	{
		EOS_Platform_SetNetworkStatus(Entry.Key, NewNetworkStatus);
		OnNetworkStatusChanged.Broadcast(OldNetworkStatus, NewNetworkStatus);
	}
}

void FEOSSDKManager::OnApplicationStatusChanged(EOS_EApplicationStatus ApplicationStatus)
{
	check(IsInGameThread());

	UE_LOGF(LogEOSShared, Log, "%s [%ls] -> [%ls]", __FUNCTION__, LexToString(CachedApplicationStatus), LexToString(ApplicationStatus));
	CachedApplicationStatus = ApplicationStatus;
	for (const TPair<EOS_HPlatform, IEOSPlatformHandleWeakPtr>& Entry : ActivePlatforms)
	{
		EOS_Platform_SetApplicationStatus(Entry.Key, ApplicationStatus);
	}
}

void FEOSSDKManager::OnLogVerbosityChanged(const FLogCategoryName& CategoryName, ELogVerbosity::Type OldVerbosity, ELogVerbosity::Type NewVerbosity)
{
#if !NO_LOGGING
	if (IsInitialized() &&
		CategoryName == LogEOSSDK.GetCategoryName())
	{
		const EOS_EResult EosResult = EOS_Logging_SetLogLevel(EOS_ELogCategory::EOS_LC_ALL_CATEGORIES, ConvertLogLevel(NewVerbosity));
		if (EosResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogEOSShared, Warning, "%s EOS_Logging_SetLogLevel Verbosity=[%ls] Result=[%ls]", __FUNCTION__, ToString(NewVerbosity), *LexToString(EosResult));
		}
	}
#endif // !NO_LOGGING
}

FString FEOSSDKManager::GetSDKVersion() const
{
	return FString((const UTF8CHAR*)EOS_GetVersion());
}

FString FEOSSDKManager::GetProductName() const
{
	FString ProductName;
	GConfig->GetString(TEXT("EOSSDK"), TEXT("ProductName"), ProductName, GEngineIni);

	if (ProductName.IsEmpty())
	{
		ProductName = FApp::GetProjectName();
	}

	if (ProductName.IsEmpty())
	{
		ProductName = TEXT("UnrealEngine");
	}

	return ProductName;
}

FString FEOSSDKManager::GetProductVersion() const
{
	return FApp::GetBuildVersion();
}

FString FEOSSDKManager::GetCacheDirBase() const
{
	if (FPlatformMisc::IsCacheStorageAvailable())
	{
		return FPlatformProcess::UserDir();
	}
	else
	{
		return FString(); 
	}
	
}

FString FEOSSDKManager::GetOverrideCountryCode(const EOS_HPlatform Platform) const
{
	FString Result;

	char CountryCode[EOS_COUNTRYCODE_MAX_LENGTH + 1];
	int32_t CountryCodeLength = sizeof(CountryCode);
	if (EOS_Platform_GetOverrideCountryCode(Platform, CountryCode, &CountryCodeLength) == EOS_EResult::EOS_Success)
	{
		Result = UTF8_TO_TCHAR(CountryCode);
	}

	return Result;
}

FString FEOSSDKManager::GetOverrideLocaleCode(const EOS_HPlatform Platform) const
{
	FString Result;

	char LocaleCode[EOS_LOCALECODE_MAX_LENGTH + 1];
	int32_t LocaleCodeLength = sizeof(LocaleCode);
	if (EOS_Platform_GetOverrideLocaleCode(Platform, LocaleCode, &LocaleCodeLength) == EOS_EResult::EOS_Success)
	{
		Result = UTF8_TO_TCHAR(LocaleCode);
	}

	return Result;
}

void FEOSSDKManager::ReleasePlatform(EOS_HPlatform PlatformHandle)
{
	check(IsInGameThread());

	if(ActivePlatforms.Contains(PlatformHandle) && !ReleasedPlatforms.Contains(PlatformHandle))
	{
		FRWScopeLock ScopeLock(ActivePlatformsCS, SLT_Write);

		ReleasedPlatforms.Emplace(PlatformHandle);
	}

	OnPreReleasePlatform.Broadcast(PlatformHandle);
}

void FEOSSDKManager::ReleaseReleasedPlatforms()
{
	check(IsInGameThread());

	if (ReleasedPlatforms.Num() > 0)
	{
		{
			FRWScopeLock ScopeLock(ActivePlatformsCS, SLT_Write);

			for (EOS_HPlatform PlatformHandle : ReleasedPlatforms)
			{
				if (ensure(ActivePlatforms.Contains(PlatformHandle)))
				{
					EOS_Platform_Release(PlatformHandle);

					ActivePlatforms.Remove(PlatformHandle);
				}
			}

			ReleasedPlatforms.Empty();
		}

		SetupTicker();
	}
}

void FEOSSDKManager::Shutdown()
{
	check(IsInGameThread());

	if (IsInitialized())
	{
		// Release already released platforms
		ReleaseReleasedPlatforms();

		if (ActivePlatforms.Num() > 0)
		{
			{
				FRWScopeLock ScopeLock(ActivePlatformsCS, SLT_Write);

				UE_LOGF(LogEOSShared, Warning, "%s Releasing %d remaining platforms", __FUNCTION__, ActivePlatforms.Num());

				TArray<EOS_HPlatform> ActivePlatformHandles;
				ActivePlatforms.GenerateKeyArray(ActivePlatformHandles);
				ReleasedPlatforms.Append(ActivePlatformHandles);
			}

			ReleaseReleasedPlatforms();
		}

		FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);

#if !NO_LOGGING
		FCoreDelegates::OnLogVerbosityChanged.RemoveAll(this);
#endif // !NO_LOGGING

		const EOS_EResult Result = EOS_Shutdown();
		UE_LOGF(LogEOSShared, Log, "%s EOS_Shutdown Result=[%ls]", __FUNCTION__, *LexToString(Result));

		CallbackObjects.Empty();

		bInitialized = false;

		FCoreDelegates::OnNetworkConnectionStatusChanged.RemoveAll(this);

#if WITH_ENGINE
		// We can't check bRenderReady at this point as Slate might have shut down already
		if (FSlateApplication::IsInitialized())
		{
			FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
			if (Renderer)
			{
				Renderer->OnBackBufferReadyToPresent().RemoveAll(this);
			}
		}
#endif
	}
}

// TODO: Remove Windows workaround when EOS stops modifying thread priority
#if PLATFORM_WINDOWS
extern "C" int __stdcall GetThreadPriority(void* hThread);
extern "C" void* __stdcall GetCurrentThread(void);
extern "C" int __stdcall SetThreadPriority(void* hThread, int nPriority);
#endif
EOS_EResult FEOSSDKManager::EOSInitialize(EOS_InitializeOptions& Options)
{
	OnPreInitializeSDK.Broadcast(Options);

	EOS_InitializeOptions* OptionsPtr = &Options;
	OnPreInitializeSDK2.Broadcast(OptionsPtr);

#if PLATFORM_WINDOWS
	void* const hThread = ::GetCurrentThread();
	const int Priority = ::GetThreadPriority(hThread);
#endif
	const EOS_EResult Result = EOS_Initialize(OptionsPtr);
#if PLATFORM_WINDOWS
	if (Priority != 0x7fffffff)
	{
		::SetThreadPriority(hThread, Priority);
	}
#endif
	return Result;
}

bool FEOSSDKManager::Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!FParse::Command(&Cmd, TEXT("EOSSDK")))
	{
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("INFO")))
	{
		LogInfo();
	}
	else if (FParse::Command(&Cmd, TEXT("DISABLENETWORK")))
	{
		for (const TPair<EOS_HPlatform, IEOSPlatformHandleWeakPtr>& Entry : ActivePlatforms)
		{
			EOS_ENetworkStatus CurrentStatus = EOS_Platform_GetNetworkStatus(Entry.Key);

			if (CurrentStatus == EOS_ENetworkStatus::EOS_NS_Online)
			{
				OnNetworkConnectionStatusChanged(ENetworkConnectionStatus::Connected, ENetworkConnectionStatus::Local); 
			}
		}
	}
	else if (FParse::Command(&Cmd, TEXT("ENABLENETWORK")))
	{
		for (const TPair<EOS_HPlatform, IEOSPlatformHandleWeakPtr>& Entry : ActivePlatforms)
		{
			EOS_ENetworkStatus CurrentStatus = EOS_Platform_GetNetworkStatus(Entry.Key);

			if (CurrentStatus == EOS_ENetworkStatus::EOS_NS_Offline)
			{
				OnNetworkConnectionStatusChanged(ENetworkConnectionStatus::Local, ENetworkConnectionStatus::Connected);
			}
		}
	}
	else
	{
		UE_LOGF(LogEOSShared, Warning, "%s Unknown exec command: %ls]", __FUNCTION__, Cmd);
	}

	return true;
}

#define UE_LOG_EOSSDK_INFO(Format, ...) UE_LOG(LogEOSShared, Log, TEXT("%hs %*s" Format), __FUNCTION__, Indent * 2, TEXT(""), ##__VA_ARGS__)

void FEOSSDKManager::LogInfo(int32 Indent) const
{
	check(IsInGameThread());

	UE_LOG_EOSSDK_INFO("ProductName=%s", *GetProductName());
	UE_LOG_EOSSDK_INFO("ProductVersion=%s", *GetProductVersion());
	UE_LOG_EOSSDK_INFO("CacheDirBase=%s", *GetCacheDirBase());
	UE_LOG_EOSSDK_INFO("Platforms=%d", ActivePlatforms.Num());

	TArray<EOS_HPlatform> ActivePlatformHandles;
	ActivePlatforms.GenerateKeyArray(ActivePlatformHandles);
	for (int32 PlatformIndex = 0; PlatformIndex < ActivePlatformHandles.Num(); PlatformIndex++)
	{
		const EOS_HPlatform Platform = ActivePlatformHandles[PlatformIndex];
		UE_LOG_EOSSDK_INFO("Platform=%d", PlatformIndex);
		Indent++;
		LogPlatformInfo(Platform, Indent);
		Indent--;
	}
}

void FEOSSDKManager::LogPlatformInfo(const EOS_HPlatform Platform, int32 Indent) const
{
	UE_LOG_EOSSDK_INFO("ApplicationStatus=%s", LexToString(EOS_Platform_GetApplicationStatus(Platform)));
	UE_LOG_EOSSDK_INFO("NetworkStatus=%s", LexToString(EOS_Platform_GetNetworkStatus(Platform)));
	UE_LOG_EOSSDK_INFO("OverrideCountryCode=%s", *GetOverrideCountryCode(Platform));
	UE_LOG_EOSSDK_INFO("OverrideLocaleCode=%s", *GetOverrideLocaleCode(Platform));

	EOS_Platform_GetDesktopCrossplayStatusOptions GetDesktopCrossplayStatusOptions;
	GetDesktopCrossplayStatusOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_GETDESKTOPCROSSPLAYSTATUS_API_LATEST, 1);

	EOS_Platform_GetDesktopCrossplayStatusInfo GetDesktopCrossplayStatusInfo;
	EOS_EResult Result = EOS_Platform_GetDesktopCrossplayStatus(Platform, &GetDesktopCrossplayStatusOptions, &GetDesktopCrossplayStatusInfo);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("DesktopCrossplayStatusInfo Status=%s ServiceInitResult=%d",
			LexToString(GetDesktopCrossplayStatusInfo.Status),
			GetDesktopCrossplayStatusInfo.ServiceInitResult);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("DesktopCrossplayStatusInfo (EOS_Platform_GetDesktopCrossplayStatus failed: %s)", *LexToString(Result));
	}

	const EOS_HAuth AuthHandle = EOS_Platform_GetAuthInterface(Platform);
	const int32_t AuthLoggedInAccountsCount = EOS_Auth_GetLoggedInAccountsCount(AuthHandle);
	UE_LOG_EOSSDK_INFO("AuthLoggedInAccounts=%d", AuthLoggedInAccountsCount);

	for (int32_t AuthLoggedInAccountIndex = 0; AuthLoggedInAccountIndex < AuthLoggedInAccountsCount; AuthLoggedInAccountIndex++)
	{
		const EOS_EpicAccountId LoggedInAccount = EOS_Auth_GetLoggedInAccountByIndex(AuthHandle, AuthLoggedInAccountIndex);
		UE_LOG_EOSSDK_INFO("AuthLoggedInAccount=%d", AuthLoggedInAccountIndex);
		Indent++;
		LogUserInfo(Platform, LoggedInAccount, LoggedInAccount, Indent);
		LogAuthInfo(Platform, LoggedInAccount, Indent);
		LogPresenceInfo(Platform, LoggedInAccount, LoggedInAccount, Indent);
		LogFriendsInfo(Platform, LoggedInAccount, Indent);
		Indent--;
	}

	const EOS_HConnect ConnectHandle = EOS_Platform_GetConnectInterface(Platform);
	const int32_t ConnectLoggedInAccountsCount = EOS_Connect_GetLoggedInUsersCount(ConnectHandle);
	UE_LOG_EOSSDK_INFO("ConnectLoggedInAccounts=%d", ConnectLoggedInAccountsCount);

	for (int32_t ConnectLoggedInAccountIndex = 0; ConnectLoggedInAccountIndex < ConnectLoggedInAccountsCount; ConnectLoggedInAccountIndex++)
	{
		const EOS_ProductUserId LoggedInAccount = EOS_Connect_GetLoggedInUserByIndex(ConnectHandle, ConnectLoggedInAccountIndex);
		UE_LOG_EOSSDK_INFO("ConnectLoggedInAccount=%d", ConnectLoggedInAccountIndex);
		Indent++;
		LogConnectInfo(Platform, LoggedInAccount, Indent);
		Indent--;
	}
}

void FEOSSDKManager::LogAuthInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent) const
{
	const EOS_HAuth AuthHandle = EOS_Platform_GetAuthInterface(Platform);
	UE_LOG_EOSSDK_INFO("LoginStatus=%s", LexToString(EOS_Auth_GetLoginStatus(AuthHandle, LoggedInAccount)));

	EOS_Auth_CopyUserAuthTokenOptions CopyUserAuthTokenOptions;
	CopyUserAuthTokenOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST, 1);

	EOS_Auth_Token* AuthToken;
	EOS_EResult Result = EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyUserAuthTokenOptions, LoggedInAccount, &AuthToken);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("AuthToken");
		Indent++;
		UE_LOG_EOSSDK_INFO("App=%s", UTF8_TO_TCHAR(AuthToken->App));
		UE_LOG_EOSSDK_INFO("ClientId=%s", UTF8_TO_TCHAR(AuthToken->ClientId));
#if !UE_BUILD_SHIPPING
		UE_LOG_EOSSDK_INFO("AccessToken=%s", UTF8_TO_TCHAR(AuthToken->AccessToken));
#endif // !UE_BUILD_SHIPPING
		UE_LOG_EOSSDK_INFO("ExpiresIn=%f", AuthToken->ExpiresIn);
		UE_LOG_EOSSDK_INFO("ExpiresAt=%s", UTF8_TO_TCHAR(AuthToken->ExpiresAt));
		UE_LOG_EOSSDK_INFO("AuthType=%s", LexToString(AuthToken->AuthType));
#if !UE_BUILD_SHIPPING
		UE_LOG_EOSSDK_INFO("RefreshToken=%s", UTF8_TO_TCHAR(AuthToken->RefreshToken));
#endif // !UE_BUILD_SHIPPING
		UE_LOG_EOSSDK_INFO("RefreshExpiresIn=%f", AuthToken->RefreshExpiresIn);
		UE_LOG_EOSSDK_INFO("RefreshExpiresAt=%s", UTF8_TO_TCHAR(AuthToken->RefreshExpiresAt));
		Indent--;
		EOS_Auth_Token_Release(AuthToken);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("AuthToken (EOS_Auth_CopyUserAuthToken failed: %s)", *LexToString(Result));
	}

#if !UE_BUILD_SHIPPING
	EOS_Auth_CopyIdTokenOptions CopyIdTokenOptions;
	CopyIdTokenOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYIDTOKEN_API_LATEST, 1);
	CopyIdTokenOptions.AccountId = LoggedInAccount;

	EOS_Auth_IdToken* IdToken;
	Result = EOS_Auth_CopyIdToken(AuthHandle, &CopyIdTokenOptions, &IdToken);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("IdToken=%s", UTF8_TO_TCHAR(IdToken->JsonWebToken));
		EOS_Auth_IdToken_Release(IdToken);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("IdToken (EOS_Auth_CopyIdToken failed: %s)", *LexToString(Result));
	}
#endif // !UE_BUILD_SHIPPING
}

void FEOSSDKManager::LogUserInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent) const
{
	UE_LOG_EOSSDK_INFO("EpicAccountId=%s", *LexToString(TargetAccount));

	EOS_UserInfo_CopyUserInfoOptions CopyUserInfoOptions;
	CopyUserInfoOptions.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYUSERINFO_API_LATEST, 3);
	CopyUserInfoOptions.LocalUserId = LoggedInAccount;
	CopyUserInfoOptions.TargetUserId = TargetAccount;

	const EOS_HUserInfo UserInfoHandle = EOS_Platform_GetUserInfoInterface(Platform);
	EOS_UserInfo* UserInfo;
	EOS_EResult Result = EOS_UserInfo_CopyUserInfo(UserInfoHandle, &CopyUserInfoOptions, &UserInfo);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("UserInfo");
		Indent++;
		UE_LOG_EOSSDK_INFO("Country=%s", UTF8_TO_TCHAR(UserInfo->Country));
		UE_LOG_EOSSDK_INFO("DisplayName=%s", UTF8_TO_TCHAR(UserInfo->DisplayName));
		UE_LOG_EOSSDK_INFO("PreferredLanguage=%s", UTF8_TO_TCHAR(UserInfo->PreferredLanguage));
		UE_LOG_EOSSDK_INFO("Nickname=%s", UTF8_TO_TCHAR(UserInfo->Nickname));
		UE_LOG_EOSSDK_INFO("DisplayNameSanitized=%s", UTF8_TO_TCHAR(UserInfo->DisplayNameSanitized));
		Indent--;

		EOS_UserInfo_Release(UserInfo);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("UserInfo (EOS_UserInfo_CopyUserInfo failed: %s)", *LexToString(Result));
	}

	EOS_UserInfo_CopyBestDisplayNameOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAME_API_LATEST, 1);
	Options.LocalUserId = LoggedInAccount;
	Options.TargetUserId = TargetAccount;

	EOS_UserInfo_BestDisplayName* BestDisplayName;
	Result = EOS_UserInfo_CopyBestDisplayName(UserInfoHandle, &Options, &BestDisplayName);

	if (Result == EOS_EResult::EOS_UserInfo_BestDisplayNameIndeterminate)
	{
		EOS_UserInfo_CopyBestDisplayNameWithPlatformOptions WithPlatformOptions = {};
		WithPlatformOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAMEWITHPLATFORM_API_LATEST, 1);
		WithPlatformOptions.LocalUserId = LoggedInAccount;
		WithPlatformOptions.TargetUserId = TargetAccount;
		WithPlatformOptions.TargetPlatformType = EOS_OPT_Epic;

		Result = EOS_UserInfo_CopyBestDisplayNameWithPlatform(UserInfoHandle, &WithPlatformOptions, &BestDisplayName);
	}

	if (Result == EOS_EResult::EOS_Success)
	{
		Indent++;
		UE_LOG_EOSSDK_INFO("BestDisplayName");
		Indent++;
		UE_LOG_EOSSDK_INFO("DisplayName=%hs", BestDisplayName->DisplayName);
		UE_LOG_EOSSDK_INFO("DisplayNameSanitized=%hs", BestDisplayName->DisplayNameSanitized);
		UE_LOG_EOSSDK_INFO("Nickname=%hs", BestDisplayName->Nickname);
		Indent -= 2;

		EOS_UserInfo_BestDisplayName_Release(BestDisplayName);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("UserInfo (BestDisplayName retrieval failed: %s)", *LexToString(Result));
	}
}

void FEOSSDKManager::LogPresenceInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent) const
{
	EOS_Presence_HasPresenceOptions HasPresenceOptions;
	HasPresenceOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_PRESENCE_HASPRESENCE_API_LATEST, 1);
	HasPresenceOptions.LocalUserId = LoggedInAccount;
	HasPresenceOptions.TargetUserId = TargetAccount;

	const EOS_HPresence PresenceHandle = EOS_Platform_GetPresenceInterface(Platform);
	if (!EOS_Presence_HasPresence(PresenceHandle, &HasPresenceOptions))
	{
		UE_LOG_EOSSDK_INFO("Presence (None)");
		return;
	}

	EOS_Presence_CopyPresenceOptions CopyPresenceOptions;
	CopyPresenceOptions.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_PRESENCE_COPYPRESENCE_API_LATEST, 3);
	CopyPresenceOptions.LocalUserId = LoggedInAccount;
	CopyPresenceOptions.TargetUserId = TargetAccount;

	EOS_Presence_Info* PresenceInfo;
	EOS_EResult Result = EOS_Presence_CopyPresence(PresenceHandle, &CopyPresenceOptions, &PresenceInfo);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("Presence");
		Indent++;
		UE_LOG_EOSSDK_INFO("Status=%s", LexToString(PresenceInfo->Status));
		UE_LOG_EOSSDK_INFO("ProductId=%s", UTF8_TO_TCHAR(PresenceInfo->ProductId));
		UE_LOG_EOSSDK_INFO("ProductName=%s", UTF8_TO_TCHAR(PresenceInfo->ProductName));
		UE_LOG_EOSSDK_INFO("ProductVersion=%s", UTF8_TO_TCHAR(PresenceInfo->ProductVersion));
		UE_LOG_EOSSDK_INFO("Platform=%s", UTF8_TO_TCHAR(PresenceInfo->Platform));
		UE_LOG_EOSSDK_INFO("IntegratedPlatform=%s", UTF8_TO_TCHAR(PresenceInfo->IntegratedPlatform));
		UE_LOG_EOSSDK_INFO("RichText=%s", UTF8_TO_TCHAR(PresenceInfo->RichText));
		UE_LOG_EOSSDK_INFO("RecordsCount=%d", PresenceInfo->RecordsCount);
		Indent++;
		for (int32_t Index = 0; Index < PresenceInfo->RecordsCount; Index++)
		{
			UE_LOG_EOSSDK_INFO("Key=%s Value=%s", UTF8_TO_TCHAR(PresenceInfo->Records[Index].Key), UTF8_TO_TCHAR(PresenceInfo->Records[Index].Value));
		}
		Indent--;
		Indent--;

		EOS_Presence_Info_Release(PresenceInfo);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("Presence (EOS_Presence_CopyPresence failed: %s)", *LexToString(Result));
	}
}

void FEOSSDKManager::LogFriendsInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent) const
{
	EOS_Friends_GetFriendsCountOptions FriendsCountOptions;
	FriendsCountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_GETFRIENDSCOUNT_API_LATEST, 1);
	FriendsCountOptions.LocalUserId = LoggedInAccount;

	const EOS_HFriends FriendsHandle = EOS_Platform_GetFriendsInterface(Platform);
	const int32_t FriendsCount = EOS_Friends_GetFriendsCount(FriendsHandle, &FriendsCountOptions);
	UE_LOG_EOSSDK_INFO("Friends=%d", FriendsCount);

	EOS_Friends_GetFriendAtIndexOptions FriendAtIndexOptions;
	FriendAtIndexOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_GETFRIENDATINDEX_API_LATEST, 1);
	FriendAtIndexOptions.LocalUserId = LoggedInAccount;

	for (int32_t FriendIndex = 0; FriendIndex < FriendsCount; FriendIndex++)
	{
		FriendAtIndexOptions.Index = FriendIndex;
		const EOS_EpicAccountId FriendId = EOS_Friends_GetFriendAtIndex(FriendsHandle, &FriendAtIndexOptions);
		UE_LOG_EOSSDK_INFO("Friend=%d", FriendIndex);
		Indent++;

		EOS_Friends_GetStatusOptions GetStatusOptions;
		GetStatusOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_GETSTATUS_API_LATEST, 1);
		GetStatusOptions.LocalUserId = LoggedInAccount;
		GetStatusOptions.TargetUserId = FriendId;

		const EOS_EFriendsStatus FriendStatus = EOS_Friends_GetStatus(FriendsHandle, &GetStatusOptions);
		UE_LOG_EOSSDK_INFO("FriendStatus=%s", LexToString(FriendStatus));

		LogUserInfo(Platform, LoggedInAccount, FriendId, Indent);
		LogPresenceInfo(Platform, LoggedInAccount, FriendId, Indent);
		Indent--;
	}
}

void FEOSSDKManager::LogConnectInfo(const EOS_HPlatform Platform, const EOS_ProductUserId LoggedInAccount, int32 Indent) const
{
	UE_LOG_EOSSDK_INFO("ProductUserId=%s", *LexToString(LoggedInAccount));

	const EOS_HConnect ConnectHandle = EOS_Platform_GetConnectInterface(Platform);
	UE_LOG_EOSSDK_INFO("LoginStatus=%s", LexToString(EOS_Connect_GetLoginStatus(ConnectHandle, LoggedInAccount)));

	EOS_EResult Result;

#if !UE_BUILD_SHIPPING
	EOS_Connect_CopyIdTokenOptions CopyIdTokenOptions;
	CopyIdTokenOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_COPYIDTOKEN_API_LATEST, 1);
	CopyIdTokenOptions.LocalUserId = LoggedInAccount;

	EOS_Connect_IdToken* IdToken;
	Result = EOS_Connect_CopyIdToken(ConnectHandle, &CopyIdTokenOptions, &IdToken);
	if (Result == EOS_EResult::EOS_Success)
	{
		UE_LOG_EOSSDK_INFO("IdToken=%s", UTF8_TO_TCHAR(IdToken->JsonWebToken));
		EOS_Connect_IdToken_Release(IdToken);
	}
	else
	{
		UE_LOG_EOSSDK_INFO("IdToken (EOS_Connect_CopyIdToken failed: %s)", *LexToString(Result));
	}
#endif // !UE_BUILD_SHIPPING

	EOS_Connect_GetProductUserExternalAccountCountOptions ExternalAccountCountOptions;
	ExternalAccountCountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_GETPRODUCTUSEREXTERNALACCOUNTCOUNT_API_LATEST, 1);
	ExternalAccountCountOptions.TargetUserId = LoggedInAccount;

	const uint32_t ExternalAccountCount = EOS_Connect_GetProductUserExternalAccountCount(ConnectHandle, &ExternalAccountCountOptions);
	UE_LOG_EOSSDK_INFO("ExternalAccounts=%d", ExternalAccountCount);

	EOS_Connect_CopyProductUserExternalAccountByIndexOptions ExternalAccountByIndexOptions;
	ExternalAccountByIndexOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_COPYPRODUCTUSEREXTERNALACCOUNTBYINDEX_API_LATEST, 1);
	ExternalAccountByIndexOptions.TargetUserId = LoggedInAccount;

	for (uint32_t ExternalAccountIndex = 0; ExternalAccountIndex < ExternalAccountCount; ExternalAccountIndex++)
	{
		UE_LOG_EOSSDK_INFO("ExternalAccount=%u", ExternalAccountIndex);
		Indent++;

		ExternalAccountByIndexOptions.ExternalAccountInfoIndex = ExternalAccountIndex;
		EOS_Connect_ExternalAccountInfo* ExternalAccountInfo;
		Result = EOS_Connect_CopyProductUserExternalAccountByIndex(ConnectHandle, &ExternalAccountByIndexOptions, &ExternalAccountInfo);
		if (Result == EOS_EResult::EOS_Success)
		{
			UE_LOG_EOSSDK_INFO("ExternalAccountInfo");
			Indent++;
			UE_LOG_EOSSDK_INFO("DisplayName=%s", UTF8_TO_TCHAR(ExternalAccountInfo->DisplayName));
			UE_LOG_EOSSDK_INFO("AccountId=%s", UTF8_TO_TCHAR(ExternalAccountInfo->AccountId));
			UE_LOG_EOSSDK_INFO("AccountIdType=%s", LexToString(ExternalAccountInfo->AccountIdType));
			UE_LOG_EOSSDK_INFO("LastLoginTime=%lld", ExternalAccountInfo->LastLoginTime);
			Indent--;

			EOS_Connect_ExternalAccountInfo_Release(ExternalAccountInfo);
		}
		else
		{
			UE_LOG_EOSSDK_INFO("ExternalAccountInfo (EOS_Connect_CopyProductUserExternalAccountByIndex failed: %s)", *LexToString(Result));
		}

		Indent--;
	}
}

void FEOSSDKManager::AddCallbackObject(TUniquePtr<FCallbackBase> CallbackObj)
{
	CallbackObjects.Emplace(MoveTemp(CallbackObj));
}

TSharedRef<IEOSFastTickLock> FEOSSDKManager::GetFastTickLock()
{
	if (TSharedPtr<IEOSFastTickLock> ExistingLock = FastTickLock.Pin())
	{
		return ExistingLock.ToSharedRef();
	}
	CSV_TRACE_REGION_BEGIN(TEXT("HighPriorityEosLock"), TEXT("OnlineTick"))
	TSharedRef<IEOSFastTickLock> NewLock = MakeShared<FEOSFastTickLock>();
	FastTickLock = NewLock;

	SetupTicker();

	return NewLock;
}

FEOSPlatformHandle::~FEOSPlatformHandle()
{
	Manager.ReleasePlatform(PlatformHandle);
}

void FEOSPlatformHandle::Tick()
{
	LLM_SCOPE(ELLMTag::RealTimeCommunications);
	QUICK_SCOPE_CYCLE_COUNTER(FEOSPlatformHandle_Tick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSSDK);
	EOS_Platform_Tick(PlatformHandle);
}

TSharedRef<IEOSFastTickLock> FEOSPlatformHandle::GetFastTickLock()
{
	return Manager.GetFastTickLock();
}

FString FEOSPlatformHandle::GetConfigName() const
{
	return ConfigName;
}

FString FEOSPlatformHandle::GetOverrideCountryCode() const
{
	return Manager.GetOverrideCountryCode(PlatformHandle);
}

FString FEOSPlatformHandle::GetOverrideLocaleCode() const
{
	return Manager.GetOverrideLocaleCode(PlatformHandle);
}

void FEOSPlatformHandle::LogInfo(int32 Indent) const
{
	Manager.LogPlatformInfo(PlatformHandle, Indent);
}

void FEOSPlatformHandle::LogAuthInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent) const
{
	Manager.LogAuthInfo(PlatformHandle, LoggedInAccount, Indent);
}

void FEOSPlatformHandle::LogUserInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent) const
{
	Manager.LogUserInfo(PlatformHandle, LoggedInAccount, TargetAccount, Indent);
}

void FEOSPlatformHandle::LogPresenceInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent) const
{
	Manager.LogPresenceInfo(PlatformHandle, LoggedInAccount, TargetAccount, Indent);
}

void FEOSPlatformHandle::LogFriendsInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent) const
{
	Manager.LogFriendsInfo(PlatformHandle, LoggedInAccount, Indent);
}

void FEOSPlatformHandle::LogConnectInfo(const EOS_ProductUserId LoggedInAccount, int32 Indent) const
{
	Manager.LogConnectInfo(PlatformHandle, LoggedInAccount, Indent);
}

FEOSFastTickLock::~FEOSFastTickLock()
{
	if (FEOSSDKManager* Manager = static_cast<FEOSSDKManager*>(IEOSSDKManager::Get()))
	{
		CSV_TRACE_REGION_END(TEXT("HighPriorityEosLock"), TEXT("OnlineTick"))
		Manager->FastTickLock.Reset();
		Manager->SetupTicker();
	}
}

#endif // WITH_EOS_SDK