// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "IEOSSDKManager.h"
#include "Misc/CoreMisc.h"

#if WITH_ENGINE
#include "Widgets/SWindow.h"
#include "Rendering/SlateRenderer.h"
#include "Slate/SlateViewportProvider.h"
#endif

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_sdk.h"
#include "eos_auth_types.h"
#include "eos_common.h"
#include "eos_connect_types.h"
#include "eos_init.h"
#include "eos_integratedplatform.h"
#include "eos_ui.h"

struct FEOSPlatformHandle;

class FEOSSDKManager :
	public IEOSSDKManager,
	public FSelfRegisteringExec
{
public:
	FEOSSDKManager();
	virtual ~FEOSSDKManager();

	// Begin IEOSSDKManager
	virtual bool IsInitialized() const override { return bInitialized; }

	virtual const FEOSSDKPlatformConfig* GetPlatformConfig(const FString& PlatformConfigName, bool bLoadIfMissing = false) override;
	virtual bool AddPlatformConfig(const FEOSSDKPlatformConfig& PlatformConfig, bool bOverwriteExistingConfig = false) override;
	virtual const FString& GetDefaultPlatformConfigName() override;
	virtual void SetDefaultPlatformConfigName(const FString& PlatformConfigName) override;

	virtual IEOSPlatformHandlePtr CreatePlatform(const FString& PlatformConfigName, FName InstanceName = NAME_None) override;
	virtual IEOSPlatformHandlePtr CreatePlatform(EOS_Platform_Options& PlatformOptions) override;
	virtual TArray<IEOSPlatformHandlePtr> GetActivePlatforms() override;

	virtual FString GetSDKVersion() const override;
	virtual FString GetProductName() const override;
	virtual FString GetProductVersion() const override;
	virtual FString GetCacheDirBase() const override;
	virtual FString GetOverrideCountryCode(const EOS_HPlatform Platform) const override;
	virtual FString GetOverrideLocaleCode(const EOS_HPlatform Platform) const override;

	virtual void LogInfo(int32 Indent = 0) const override;
	virtual void LogPlatformInfo(const EOS_HPlatform Platform, int32 Indent = 0) const override;
	virtual void LogAuthInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const override;
	virtual void LogUserInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const override;
	virtual void LogPresenceInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const override;
	virtual void LogFriendsInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const override;
	virtual void LogConnectInfo(const EOS_HPlatform Platform, const EOS_ProductUserId LoggedInAccount, int32 Indent = 0) const override;

	virtual void AddCallbackObject(TUniquePtr<class FCallbackBase> CallbackObj) override;

	virtual TSharedRef<IEOSFastTickLock> GetFastTickLock() override;
	// End IEOSSDKManager

	virtual EOS_EResult Initialize();
	void Shutdown();

protected:
	// Begin FSelfRegisteringExec
	virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	// End FSelfRegisteringExec

	virtual EOS_EResult EOSInitialize(EOS_InitializeOptions& Options);
	virtual IEOSPlatformHandlePtr CreatePlatform(const FEOSSDKPlatformConfig& PlatformConfig, EOS_Platform_Options& PlatformOptions);
	virtual bool Tick(float);
	virtual const void* GetIntegratedPlatformOptions();
	virtual EOS_IntegratedPlatformType GetIntegratedPlatformType();

#if WITH_ENGINE
	/** Provided to `OnBackBufferReadyToPresent` to get access to the render thread. */
	virtual void OnBackBufferReady_RenderThread(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);
	/**
	 * Check that the overlay is ready to be rendered.
	 * This will also add the Back Buffer Ready To Present handler.
	 */
	virtual bool IsRenderReady();
#endif

	static void OnDisplaySettingsUpdated(const EOS_UI_OnDisplaySettingsUpdatedCallbackInfo* Data);
	void RegisterDisplaySettingsUpdatedCallback(const EOS_HPlatform PlatformHandle);

	void SetInvokeOverlayButton(const EOS_HPlatform PlatformHandle);
	void ApplyOverlayPlatformOptions(EOS_Platform_Options& PlatformOptions);
	EOS_HIntegratedPlatformOptionsContainer CreateIntegratedPlatformOptionsContainer();
	void ApplyIntegratedPlatformOptions(EOS_HIntegratedPlatformOptionsContainer& Container);
	virtual void ApplySystemSpecificOptions(const void*& SystemSpecificOptions);
	void CallUIPrePresent(const EOS_UI_PrePresentOptions& Options);

	static EOS_ENetworkStatus ConvertNetworkStatus(ENetworkConnectionStatus Status);
	void OnNetworkConnectionStatusChanged(ENetworkConnectionStatus LastConnectionState, ENetworkConnectionStatus ConnectionState);
	void OnApplicationStatusChanged(EOS_EApplicationStatus ApplicationStatus);
	EOS_EApplicationStatus CachedApplicationStatus = EOS_EApplicationStatus::EOS_AS_Foreground;

	friend struct FEOSPlatformHandle;
	friend struct FEOSFastTickLock;

	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionName);
	void LoadConfig();
	void ReleasePlatform(EOS_HPlatform PlatformHandle);
	void ReleaseReleasedPlatforms();
	void SetupTicker();
	void OnLogVerbosityChanged(const FLogCategoryName& CategoryName, ELogVerbosity::Type OldVerbosity, ELogVerbosity::Type NewVerbosity);

#if EOSSDK_RUNTIME_LOAD_REQUIRED
	void LoadSDKHandle();
	void* SDKHandle = nullptr;
#endif

	/** Critical Section to make sure the ActivePlatforms and ReleasedPlatforms arrays are thread safe */
	mutable FRWLock ActivePlatformsCS;

	/** Are we currently initialized */
	bool bInitialized = false;
	/** Tracks if the render init has completed. */
	bool bRenderReady = false;
	/** Index of the last ticked platform, used for round-robin ticking when ConfigTickIntervalSeconds > 0 */
	uint8 PlatformTickIdx = 0;
	/** Created platforms actively ticking */
	TMap<EOS_HPlatform, IEOSPlatformHandleWeakPtr> ActivePlatforms;
	/** Contains platforms released with ReleasePlatform, which we will release on the next Tick. */
	TArray<EOS_HPlatform> ReleasedPlatforms;

	/** Handle to ticker delegate for Tick(), valid whenever there are ActivePlatforms to tick, or ReleasedPlatforms to release. */
	FTSTicker::FDelegateHandle TickerHandle;
	/** Callback objects, to be released after EOS_Shutdown */
	TArray<TUniquePtr<FCallbackBase>> CallbackObjects;
	/** Cache of named platform configs that have been loaded from ini files or added manually. */
	TMap<FString, FEOSSDKPlatformConfig> PlatformConfigs;
	/** Default platform config name to use. */
	FString DefaultPlatformConfigName;
	/** Cache of named platform handles that have been created. */
	TMap<FString, TMap<FName, IEOSPlatformHandleWeakPtr>> PlatformHandles;
	/** If this is set, then we should ignore the config tick rate and tick at full speed. */
	TWeakPtr<IEOSFastTickLock> FastTickLock;

	/** Begin Config */
	/** Interval between platform ticks. 0 means we tick every frame. */
	double ConfigTickIntervalSeconds = 0.f;

	/** Whether or not the integrated platform options container will be set at platform creation time */
	bool bEnablePlatformIntegration = false;

	/** Whether or not to integrate with the overlay (forward inputs, set up renderer callbacks etc) */
	bool bEnableOverlayIntegration = false;

	/** Button combination to bring up the overlay (only used in certain platforms) */
	EOS_UI_EInputStateButtonFlags InvokeOverlayButtonCombination;

	/** Management flags passed on as options in integrated platform setup */
	EOS_EIntegratedPlatformManagementFlags IntegratedPlatformManagementFlags = {};
	/** End Config */
};

struct FEOSPlatformHandle : public IEOSPlatformHandle
{
	FEOSPlatformHandle(FEOSSDKManager& InManager, EOS_HPlatform InPlatformHandle)
		: IEOSPlatformHandle(InPlatformHandle), Manager(InManager)
	{
	}

	virtual ~FEOSPlatformHandle();
	virtual void Tick() override;
	virtual TSharedRef<IEOSFastTickLock> GetFastTickLock() override;

	virtual FString GetConfigName() const override;
	virtual FString GetOverrideCountryCode() const override;
	virtual FString GetOverrideLocaleCode() const override;

	virtual void LogInfo(int32 Indent = 0) const override;
	virtual void LogAuthInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const override;
	virtual void LogUserInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const override;
	virtual void LogPresenceInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const override;
	virtual void LogFriendsInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const override;
	virtual void LogConnectInfo(const EOS_ProductUserId LoggedInAccount, int32 Indent = 0) const override;

	/* Reference to the EOSSDK manager */
	FEOSSDKManager& Manager;

	/* The name of the config used to instantiate this handle */
	FString ConfigName;
};

struct FEOSFastTickLock : public IEOSFastTickLock
{
	virtual ~FEOSFastTickLock();
};

#endif // WITH_EOS_SDK