// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EOS_SDK

#include "EOSShared.h"

#include "String/ParseTokens.h"
#include "EOSSharedTypes.h"

// The EOS SDK lives in a separate DLL and is never compiled with AutoRTFM, so must always be accessed from the open.
// Any API usage in this file must always be wrapped in an open block. The open block must not make any writes to the 
// transactional heap, and must not allocate memory which could persist past the end of the block; scoping its work
// as tightly as possible is important.
#include "AutoRTFM.h"

#include "eos_auth_types.h"
#include "eos_friends_types.h"
#include "eos_p2p_types.h"
#include "eos_presence_types.h"
#include "eos_rtc_audio_types.h"
#include "eos_rtc_types.h"
#include "eos_userinfo_types.h"

DEFINE_LOG_CATEGORY(LogEOSSDK);
DEFINE_LOG_CATEGORY(LogEOSShared);

UE_AUTORTFM_ALWAYS_OPEN
const char* LexToUtf8String(const EOS_EResult EosResult)
{
	return EOS_EResult_ToString(EosResult);
}

FString LexToString(const EOS_EResult EosResult)
{
	return FString(UTF8_TO_TCHAR(LexToUtf8String(EosResult)));
}

UE_AUTORTFM_ALWAYS_OPEN
static bool ProductUserIdToUtf8String(const EOS_ProductUserId UserId, char* ProductIdString, int32_t BufferSize)
{
	return 
		EOS_ProductUserId_IsValid(UserId) == EOS_TRUE &&
		EOS_ProductUserId_ToString(UserId, ProductIdString, &BufferSize) == EOS_EResult::EOS_Success;
}

FString LexToString(const EOS_ProductUserId UserId)
{
	FString Result;

	char ProductUserIdString[EOS_PRODUCTUSERID_MAX_LENGTH + 1];
	ProductUserIdString[0] = '\0';

	if (ProductUserIdToUtf8String(UserId, ProductUserIdString, sizeof(ProductUserIdString)))
	{
		Result = UTF8_TO_TCHAR(ProductUserIdString);
	}

	return Result;
}

UE_AUTORTFM_ALWAYS_OPEN
EOS_ProductUserId ProductUserIdFromUtf8String(const char* Utf8String)
{
	return EOS_ProductUserId_FromString(Utf8String);
}

void LexFromString(EOS_ProductUserId& UserId, const TCHAR* String)
{
	UserId = ProductUserIdFromUtf8String(TCHAR_TO_UTF8(String));
}

UE_AUTORTFM_ALWAYS_OPEN
static bool EpicAccountIdToUtf8String(const EOS_EpicAccountId AccountId, char* AccountIdString, int32_t BufferSize)
{
	return 
		EOS_EpicAccountId_IsValid(AccountId) == EOS_TRUE &&
		EOS_EpicAccountId_ToString(AccountId, AccountIdString, &BufferSize) == EOS_EResult::EOS_Success;
}

FString LexToString(const EOS_EpicAccountId AccountId)
{
	FString Result;

	char AccountIdString[EOS_EPICACCOUNTID_MAX_LENGTH + 1];
	AccountIdString[0] = '\0';

	if (EpicAccountIdToUtf8String(AccountId, AccountIdString, sizeof(AccountIdString)))
	{
		Result = UTF8_TO_TCHAR(AccountIdString);
	}

	return Result;
}

UE_AUTORTFM_ALWAYS_OPEN
EOS_EpicAccountId EpicAccountIdFromUtf8String(const char* Utf8String)
{
	return EOS_EpicAccountId_FromString(Utf8String);
}

void LexFromString(EOS_EpicAccountId& AccountId, const TCHAR* String)
{
	AccountId = EpicAccountIdFromUtf8String(TCHAR_TO_UTF8(String));
}

const TCHAR* LexToString(const EOS_EApplicationStatus ApplicationStatus)
{
	switch (ApplicationStatus)
	{
		case EOS_EApplicationStatus::EOS_AS_BackgroundConstrained:		return TEXT("BackgroundConstrained");
		case EOS_EApplicationStatus::EOS_AS_BackgroundUnconstrained:	return TEXT("BackgroundUnconstrained");
		case EOS_EApplicationStatus::EOS_AS_BackgroundSuspended:		return TEXT("BackgroundSuspended");
		case EOS_EApplicationStatus::EOS_AS_Foreground:					return TEXT("Foreground");
		default: checkNoEntry();										return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_EAuthTokenType AuthTokenType)
{
	switch (AuthTokenType)
	{
		case EOS_EAuthTokenType::EOS_ATT_Client:	return TEXT("Client");
		case EOS_EAuthTokenType::EOS_ATT_User:		return TEXT("User");
		default: checkNoEntry();					return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_EDesktopCrossplayStatus DesktopCrossplayStatus)
{
	switch (DesktopCrossplayStatus)
	{
		case EOS_EDesktopCrossplayStatus::EOS_DCS_OK:							return TEXT("OK");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_ApplicationNotBootstrapped:	return TEXT("ApplicationNotBootstrapped");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_ServiceNotInstalled:			return TEXT("ServiceNotInstalled");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_ServiceStartFailed:			return TEXT("ServiceStartFailed");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_ServiceNotRunning:			return TEXT("ServiceNotRunning");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_OverlayDisabled:				return TEXT("OverlayDisabled");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_OverlayNotInstalled:			return TEXT("OverlayNotInstalled");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_OverlayTrustCheckFailed:		return TEXT("OverlayTrustCheckFailed");
		case EOS_EDesktopCrossplayStatus::EOS_DCS_OverlayLoadFailed:			return TEXT("OverlayLoadFailed");
		default: checkNoEntry();												return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_EExternalAccountType ExternalAccountType)
{
	switch (ExternalAccountType)
	{
		case EOS_EExternalAccountType::EOS_EAT_EPIC:		return TEXT("Epic");
		case EOS_EExternalAccountType::EOS_EAT_STEAM:		return TEXT("Steam");
		case EOS_EExternalAccountType::EOS_EAT_PSN:			return TEXT("PSN");
		case EOS_EExternalAccountType::EOS_EAT_XBL:			return TEXT("XBL");
		case EOS_EExternalAccountType::EOS_EAT_DISCORD:		return TEXT("Discord");
		case EOS_EExternalAccountType::EOS_EAT_GOG:			return TEXT("GOG");
		case EOS_EExternalAccountType::EOS_EAT_NINTENDO:	return TEXT("Nintendo");
		case EOS_EExternalAccountType::EOS_EAT_UPLAY:		return TEXT("UPlay");
		case EOS_EExternalAccountType::EOS_EAT_OPENID:		return TEXT("OpenID");
		case EOS_EExternalAccountType::EOS_EAT_APPLE:		return TEXT("Apple");
		case EOS_EExternalAccountType::EOS_EAT_GOOGLE:		return TEXT("Google");
		case EOS_EExternalAccountType::EOS_EAT_OCULUS:		return TEXT("Oculus");
		case EOS_EExternalAccountType::EOS_EAT_ITCHIO:		return TEXT("ItchIO");
		case EOS_EExternalAccountType::EOS_EAT_AMAZON:		return TEXT("Amazon");
		default: checkNoEntry();							return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_EFriendsStatus FriendStatus)
{
	switch (FriendStatus)
	{
		default: checkNoEntry(); // Intentional fall through
		case EOS_EFriendsStatus::EOS_FS_NotFriends:		return TEXT("NotFriends");
		case EOS_EFriendsStatus::EOS_FS_InviteSent:		return TEXT("InviteSent");
		case EOS_EFriendsStatus::EOS_FS_InviteReceived: return TEXT("InviteReceived");
		case EOS_EFriendsStatus::EOS_FS_Friends:		return TEXT("Friends");
	}
}

const TCHAR* LexToString(const EOS_ELoginStatus LoginStatus)
{
	switch (LoginStatus)
	{
		case EOS_ELoginStatus::EOS_LS_NotLoggedIn:			return TEXT("NotLoggedIn");
		case EOS_ELoginStatus::EOS_LS_UsingLocalProfile:	return TEXT("UsingLocalProfile");
		case EOS_ELoginStatus::EOS_LS_LoggedIn:				return TEXT("LoggedIn");
		default: checkNoEntry();							return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_ENetworkStatus NetworkStatus)
{
	switch (NetworkStatus)
	{
		case EOS_ENetworkStatus::EOS_NS_Disabled:	return TEXT("Disabled");
		case EOS_ENetworkStatus::EOS_NS_Offline:	return TEXT("Offline");
		case EOS_ENetworkStatus::EOS_NS_Online:		return TEXT("Online");
		default: checkNoEntry();					return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_Presence_EStatus PresenceStatus)
{
	switch (PresenceStatus)
	{
		case EOS_Presence_EStatus::EOS_PS_Offline:		return TEXT("Offline");
		case EOS_Presence_EStatus::EOS_PS_Online:		return TEXT("Online");
		case EOS_Presence_EStatus::EOS_PS_Away:			return TEXT("Away");
		case EOS_Presence_EStatus::EOS_PS_ExtendedAway:	return TEXT("ExtendedAway");
		case EOS_Presence_EStatus::EOS_PS_DoNotDisturb:	return TEXT("DoNotDisturb");
		default: checkNoEntry();						return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_EExternalCredentialType ExternalCredentialType)
{
	switch (ExternalCredentialType)
	{
		case EOS_EExternalCredentialType::EOS_ECT_AMAZON_ACCESS_TOKEN:	return TEXT("AmazonAccessToken");
		case EOS_EExternalCredentialType::EOS_ECT_APPLE_ID_TOKEN:		return TEXT("AppleIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_DEVICEID_ACCESS_TOKEN:return TEXT("DeviceIdAccessToken");
		case EOS_EExternalCredentialType::EOS_ECT_DISCORD_ACCESS_TOKEN:	return TEXT("DiscordAccessToken");
		case EOS_EExternalCredentialType::EOS_ECT_EPIC:					return TEXT("Epic");
		case EOS_EExternalCredentialType::EOS_ECT_EPIC_ID_TOKEN:		return TEXT("EpicIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_GOG_SESSION_TICKET:	return TEXT("GOGSessionTicket");
		case EOS_EExternalCredentialType::EOS_ECT_GOOGLE_ID_TOKEN:		return TEXT("GoogleIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_ITCHIO_JWT:			return TEXT("ITCHIOJWT");
		case EOS_EExternalCredentialType::EOS_ECT_ITCHIO_KEY:			return TEXT("ITCHIOKey");
		case EOS_EExternalCredentialType::EOS_ECT_NINTENDO_ID_TOKEN:	return TEXT("NintendoIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_NINTENDO_NSA_ID_TOKEN:return TEXT("NintendoNSAIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_OCULUS_USERID_NONCE:	return TEXT("OculusUserIdNonce");
		case EOS_EExternalCredentialType::EOS_ECT_OPENID_ACCESS_TOKEN:	return TEXT("OpenIdAccessToken");
		case EOS_EExternalCredentialType::EOS_ECT_PSN_ID_TOKEN:			return TEXT("PSNIdToken");
		case EOS_EExternalCredentialType::EOS_ECT_STEAM_APP_TICKET:		return TEXT("SteamAppTicket");
		case EOS_EExternalCredentialType::EOS_ECT_STEAM_SESSION_TICKET:	return TEXT("SteamSessionTicket");
		case EOS_EExternalCredentialType::EOS_ECT_UPLAY_ACCESS_TOKEN:	return TEXT("UplayAccessToken");
		case EOS_EExternalCredentialType::EOS_ECT_VIVEPORT_USER_TOKEN:	return TEXT("ViveportUserToken");
		case EOS_EExternalCredentialType::EOS_ECT_XBL_XSTS_TOKEN:		return TEXT("XBLXSTSToken");
		default: checkNoEntry();										return TEXT("Unknown");
	}
}

const TCHAR* LexToString(const EOS_ERTCAudioInputStatus Status)
{
	switch (Status)
	{
		case EOS_ERTCAudioInputStatus::EOS_RTCAIS_Idle:						return TEXT("EOS_RTCAIS_Idle");
		case EOS_ERTCAudioInputStatus::EOS_RTCAIS_Recording: 				return TEXT("EOS_RTCAIS_Recording");
		case EOS_ERTCAudioInputStatus::EOS_RTCAIS_RecordingSilent: 			return TEXT("EOS_RTCAIS_RecordingSilent");
		case EOS_ERTCAudioInputStatus::EOS_RTCAIS_RecordingDisconnected:	return TEXT("EOS_RTCAIS_RecordingDisconnected");
		case EOS_ERTCAudioInputStatus::EOS_RTCAIS_Failed:					return TEXT("EOS_RTCAIS_Failed");
		default: checkNoEntry();											return TEXT("Unknown");
	}
}

bool LexFromString(EOS_EExternalAccountType& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("Amazon")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_AMAZON;
	}
	else if (FCString::Stricmp(InString, TEXT("Apple")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_APPLE;
	}
	else if (FCString::Stricmp(InString, TEXT("Discord")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_DISCORD;
	}
	else if (FCString::Stricmp(InString, TEXT("Epic")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_EPIC;
	}
	else if (FCString::Stricmp(InString, TEXT("GOG")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_GOG;
	}
	else if (FCString::Stricmp(InString, TEXT("Google")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_GOOGLE;
	}
	else if (FCString::Stricmp(InString, TEXT("ItchIO")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_ITCHIO;
	}
	else if (FCString::Stricmp(InString, TEXT("Nintendo")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_NINTENDO;
	}
	else if (FCString::Stricmp(InString, TEXT("Oculus")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_OCULUS;
	}
	else if (FCString::Stricmp(InString, TEXT("OpenID")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_OPENID;
	}
	else if (FCString::Stricmp(InString, TEXT("PSN")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_PSN;
	}
	else if (FCString::Stricmp(InString, TEXT("Steam")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_STEAM;
	}
	else if (FCString::Stricmp(InString, TEXT("UPlay")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_UPLAY;
	}
	else if (FCString::Stricmp(InString, TEXT("XBL")) == 0)
	{
		OutEnum = EOS_EExternalAccountType::EOS_EAT_XBL;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

bool LexFromString(EOS_EAuthScopeFlags& OutEnum, const FStringView& InString)
{
	OutEnum = EOS_EAuthScopeFlags::EOS_AS_NoFlags;
	bool bParsedOk = true;

	using namespace UE::String;
	const EParseTokensOptions ParseOptions = EParseTokensOptions::SkipEmpty | EParseTokensOptions::Trim;
	auto ParseFunc = [&OutEnum, &bParsedOk](FStringView Token)
	{
		if (Token == TEXT("BasicProfile"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_BasicProfile;
		}
		else if (Token == TEXT("FriendsList"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_FriendsList;
		}
		else if (Token == TEXT("Presence"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_Presence;
		}
		else if (Token == TEXT("FriendsManagement"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_FriendsManagement;
		}
		else if (Token == TEXT("Email"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_Email;
		}
		else if (Token == TEXT("Country"))
		{
			OutEnum |= EOS_EAuthScopeFlags::EOS_AS_Country;
		}
		else
		{
			checkNoEntry();
			bParsedOk = false;
		}
	};

	ParseTokens(InString, TCHAR('|'), (TFunctionRef<void(FStringView)>)ParseFunc, ParseOptions);

	return bParsedOk;
}

bool LexFromString(EOS_ELoginCredentialType& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("ExchangeCode")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_ExchangeCode;
	}
	else if (FCString::Stricmp(InString, TEXT("PersistentAuth")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_PersistentAuth;
	}
	else if (FCString::Stricmp(InString, TEXT("Password")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_Password;
	}
	else if (FCString::Stricmp(InString, TEXT("Developer")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_Developer;
	}
	else if (FCString::Stricmp(InString, TEXT("RefreshToken")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_RefreshToken;
	}
	else if (FCString::Stricmp(InString, TEXT("AccountPortal")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_AccountPortal;
	}
	else if (FCString::Stricmp(InString, TEXT("ExternalAuth")) == 0)
	{
		OutEnum = EOS_ELoginCredentialType::EOS_LCT_ExternalAuth;
	}
	else
	{
		return false;
	}
	return true;
}

FString GetBestDisplayNameStr(const EOS_UserInfo_BestDisplayName& BestDisplayName)
{
	return FString(UTF8_TO_TCHAR(BestDisplayName.Nickname ? BestDisplayName.Nickname
		: BestDisplayName.DisplayNameSanitized ? BestDisplayName.DisplayNameSanitized
			: BestDisplayName.DisplayName));
}

FString GetUserInfoDisplayNameStr(const EOS_UserInfo& UserInfo)
{
	return FString(UTF8_TO_TCHAR(UserInfo.Nickname ? UserInfo.Nickname
		: (UserInfo.DisplayNameSanitized ? UserInfo.DisplayNameSanitized
			: UserInfo.DisplayName)));
}

bool LexFromString(EOS_ERTCBackgroundMode& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("LeaveRooms")) == 0)
	{
		OutEnum = EOS_ERTCBackgroundMode::EOS_RTCBM_LeaveRooms;
	}
	else if (FCString::Stricmp(InString, TEXT("KeepRoomsAlive")) == 0)
	{
		OutEnum = EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

//TODO: Add support for multiple flags set
bool LexFromString(EOS_UI_EInputStateButtonFlags& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("DPad_Left")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_DPad_Left;
	}
	else if (FCString::Stricmp(InString, TEXT("DPad_Right")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_DPad_Right;
	}
	else if (FCString::Stricmp(InString, TEXT("DPad_Down")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_DPad_Down;
	}
	else if (FCString::Stricmp(InString, TEXT("DPad_Up")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_DPad_Up;
	}
	else if (FCString::Stricmp(InString, TEXT("FaceButton_Left")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_FaceButton_Left;
	}
	else if (FCString::Stricmp(InString, TEXT("FaceButton_Right")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_FaceButton_Right;
	}
	else if (FCString::Stricmp(InString, TEXT("FaceButton_Bottom")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_FaceButton_Bottom;
	}
	else if (FCString::Stricmp(InString, TEXT("FaceButton_Top")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_FaceButton_Top;
	}
	else if (FCString::Stricmp(InString, TEXT("LeftShoulder")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_LeftShoulder;
	}
	else if (FCString::Stricmp(InString, TEXT("RightShoulder")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_RightShoulder;
	}
	else if (FCString::Stricmp(InString, TEXT("LeftTrigger")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_LeftTrigger;
	}
	else if (FCString::Stricmp(InString, TEXT("RightTrigger")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_RightTrigger;
	}
	else if (FCString::Stricmp(InString, TEXT("Special_Left")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_Special_Left;
	}
	else if (FCString::Stricmp(InString, TEXT("Special_Right")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_Special_Right;
	}
	else if (FCString::Stricmp(InString, TEXT("LeftThumbstick")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_LeftThumbstick;
	}
	else if (FCString::Stricmp(InString, TEXT("RightThumbstick")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_RightThumbstick;
	}
	else if (FCString::Stricmp(InString, TEXT("None")) == 0)
	{
		OutEnum = EOS_UI_EInputStateButtonFlags::EOS_UISBF_None;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

bool LexFromString(EOS_EExternalCredentialType& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("AmazonAccessToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_AMAZON_ACCESS_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("AppleIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_APPLE_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("DeviceIdAccessToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_DEVICEID_ACCESS_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("DiscordAccessToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_DISCORD_ACCESS_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("Epic")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_EPIC;
	}
	else if (FCString::Stricmp(InString, TEXT("EpicIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_EPIC_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("GOGSessionTicket")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_GOG_SESSION_TICKET;
	}
	else if (FCString::Stricmp(InString, TEXT("GoogleIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_GOOGLE_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("ITCHIOJWT")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_ITCHIO_JWT;
	}
	else if (FCString::Stricmp(InString, TEXT("ITCHIOKey")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_ITCHIO_KEY;
	}
	else if (FCString::Stricmp(InString, TEXT("NintendoIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_NINTENDO_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("NintendoNSAIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_NINTENDO_NSA_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("OculusUserIdNonce")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_OCULUS_USERID_NONCE;
	}
	else if (FCString::Stricmp(InString, TEXT("OpenIdAccessToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_OPENID_ACCESS_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("PSNIdToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_PSN_ID_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("SteamAppTicket")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_STEAM_APP_TICKET;
	}
	else if (FCString::Stricmp(InString, TEXT("SteamSessionTicket")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_STEAM_SESSION_TICKET;
	}
	else if (FCString::Stricmp(InString, TEXT("UplayAccessToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_UPLAY_ACCESS_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("ViveportUserToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_VIVEPORT_USER_TOKEN;
	}
	else if (FCString::Stricmp(InString, TEXT("XBLXSTSToken")) == 0)
	{
		OutEnum = EOS_EExternalCredentialType::EOS_ECT_XBL_XSTS_TOKEN;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

bool LexFromString(EOS_EIntegratedPlatformManagementFlags& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("ApplicationManagedIdentityLogin")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_ApplicationManagedIdentityLogin;
	}
	else if (FCString::Stricmp(InString, TEXT("Disabled")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_Disabled;
	}
	else if (FCString::Stricmp(InString, TEXT("DisablePresenceMirroring")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_DisablePresenceMirroring;
	}
	else if (FCString::Stricmp(InString, TEXT("DisableSDKManagedSessions")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_DisableSDKManagedSessions;
	}
	else if (FCString::Stricmp(InString, TEXT("LibraryManagedByApplication")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_LibraryManagedByApplication;
	}
	else if (FCString::Stricmp(InString, TEXT("LibraryManagedBySDK")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_LibraryManagedBySDK;
	}
	else if (FCString::Stricmp(InString, TEXT("PreferEOSIdentity")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_PreferEOSIdentity;
	}
	else if (FCString::Stricmp(InString, TEXT("PreferIntegratedIdentity")) == 0)
	{
		OutEnum = EOS_EIntegratedPlatformManagementFlags::EOS_IPMF_PreferIntegratedIdentity;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

bool LexFromString(EOS_EPacketReliability& OutEnum, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("UnreliableUnordered")) == 0)
	{
		OutEnum = EOS_EPacketReliability::EOS_PR_UnreliableUnordered;
	}
	else if (FCString::Stricmp(InString, TEXT("ReliableUnordered")) == 0)
	{
		OutEnum = EOS_EPacketReliability::EOS_PR_ReliableUnordered;
	}
	else if (FCString::Stricmp(InString, TEXT("ReliableOrdered")) == 0)
	{
		OutEnum = EOS_EPacketReliability::EOS_PR_ReliableOrdered;
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}

EOS_OnlinePlatformType EOSOnlinePlatformTypeFromString(const FStringView& InString)
{
	if (InString == TEXT("Unknown"))
	{
		return EOS_OPT_Unknown;
	}
	else if (InString == TEXT("Epic"))
	{	
		return EOS_OPT_Epic;
	}
	else if (InString == TEXT("Steam"))
	{	
		return EOS_OPT_Steam;
	}
	else if (InString == TEXT("PSN"))
	{	
		return EOS_OPT_PSN;
	}
	else if (InString == TEXT("Switch"))
	{	
		return EOS_OPT_Nintendo;
	}
	else if (InString == TEXT("XBL"))
	{	
		return EOS_OPT_XBL;
	}
	else
	{
		checkNoEntry();
		return EOS_OPT_Unknown;
	}
}

FString LexToString(const EOS_RTC_Option& Option)
{
	UE_EOS_CHECK_API_MISMATCH(EOS_RTC_OPTION_API_LATEST, 1);
	check(Option.ApiVersion == 1);
	return FString::Printf(TEXT("\"%hs\"=\"%hs\""), Option.Key, Option.Value);
}

#endif // WITH_EOS_SDK