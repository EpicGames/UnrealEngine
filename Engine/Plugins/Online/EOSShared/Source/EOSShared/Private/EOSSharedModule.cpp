// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSSharedModule.h"

#include "Features/IModularFeatures.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "CoreGlobals.h"
#include "EOSShared.h"

#include COMPILED_PLATFORM_HEADER(EOSSDKManager.h)

#define LOCTEXT_NAMESPACE "EOS"

#define CONFIG_SECTION_NAME TEXT("EOSSDK")

IMPLEMENT_MODULE(FEOSSharedModule, EOSShared);

void FEOSSharedModule::StartupModule()
{
#if WITH_EOS_SDK
	const bool bNoEOS = FParse::Param(FCommandLine::Get(), TEXT("NoEOS"));
	const bool bAlreadyRegistered = IModularFeatures::Get().IsModularFeatureAvailable(IEOSSDKManager::GetModularFeatureName());
	if (bNoEOS || bAlreadyRegistered)
	{
		return;
	}

	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FEOSSharedModule::OnConfigSectionsChanged);
	LoadConfig();

	SDKManager = MakeUnique<FPlatformEOSSDKManager>();
	check(SDKManager);

	IModularFeatures::Get().RegisterModularFeature(IEOSSDKManager::GetModularFeatureName(), SDKManager.Get());

	// Load from a configurable array of modules at this point, so things that need to bind to the SDK Manager init hooks can do so.
	TArray<FString> ModulesToLoad;
	GConfig->GetArray(TEXT("EOSShared"), TEXT("ModulesToLoad"), ModulesToLoad, GEngineIni);
	for (const FString& ModuleToLoad : ModulesToLoad)
	{
		if (FModuleManager::Get().ModuleExists(*ModuleToLoad))
		{
			FModuleManager::Get().LoadModule(*ModuleToLoad);
		}
	}

	// When the process starts, initialize the EOSSDK as soon as possible. SDK initialization
	// modifies environment variables which may cause a crash if another thread is iterating them
	// at the same time.
	// The SDK does not support forking - When forking is enabled, the parent process must not
	// initialize the SDK so that the child process can initialize it properly.
	if (FForkProcessHelper::IsForkRequested())
	{
		// The process has forking enabled. Wait for the fork to occur before initializing the SDK.
		// Only the child process should initialize the SDK so the parent can continue spawning
		// processes.
		OnPostForkDelegateHandle = FCoreDelegates::OnPostFork.AddLambda([](EForkProcessRole ProcessRole)
		{
			if (ProcessRole == EForkProcessRole::Child)
			{
				if (FEOSSDKManager* SDKManagerPtr = static_cast<FEOSSDKManager*>(FEOSSDKManager::Get()))
				{
					SDKManagerPtr->Initialize();
				}
			}
		});
	}
	else
	{
		if (SDKManager.IsValid())
		{
			SDKManager->Initialize();
		}
	}
#endif // WITH_EOS_SDK
}

void FEOSSharedModule::ShutdownModule()
{
#if WITH_EOS_SDK
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
	FCoreDelegates::OnPostFork.Remove(OnPostForkDelegateHandle);

	if (SDKManager.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IEOSSDKManager::GetModularFeatureName(), SDKManager.Get());
		SDKManager->Shutdown();
		SDKManager.Reset();
	}
#endif // WITH_EOS_SDK
}

FEOSSharedModule* FEOSSharedModule::Get()
{
	return FModuleManager::GetModulePtr<FEOSSharedModule>("EOSShared");
}

void FEOSSharedModule::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == GEngineIni && SectionNames.Contains(CONFIG_SECTION_NAME))
	{
		LoadConfig();
	}
}

void FEOSSharedModule::LoadConfig()
{
	GConfig->GetArray(CONFIG_SECTION_NAME, TEXT("SuppressedLogStrings"), SuppressedLogStrings_Log, GEngineIni);
	GConfig->GetArray(CONFIG_SECTION_NAME, TEXT("SuppressedLogCategories"), SuppressedLogCategories_Log, GEngineIni);
	GConfig->GetArray(CONFIG_SECTION_NAME, TEXT("SuppressedLogStrings_VeryVerbose"), SuppressedLogStrings_VeryVerbose, GEngineIni);
	GConfig->GetArray(CONFIG_SECTION_NAME, TEXT("SuppressedLogCategories_VeryVerbose"), SuppressedLogCategories_VeryVerbose, GEngineIni);
}

#undef CONFIG_SECTION_NAME
#undef LOCTEXT_NAMESPACE