// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/Fork.h"
#include "Modules/ModuleInterface.h"

#include "EOSSDKManager.h"

class FEOSSharedModule: public IModuleInterface
{
public:
	FEOSSharedModule() = default;
	~FEOSSharedModule() = default;

	static FEOSSharedModule* Get();

	const TArray<FString>& GetSuppressedLogStrings_Log() const { return SuppressedLogStrings_Log; }
	const TArray<FString>& GetSuppressedLogCategories_Log() const { return SuppressedLogCategories_Log; }
	const TArray<FString>& GetSuppressedLogStrings_VeryVerbose() const { return SuppressedLogStrings_VeryVerbose; }
	const TArray<FString>& GetSuppressedLogCategories_VeryVerbose() const { return SuppressedLogCategories_VeryVerbose; }
private:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);
	void LoadConfig();


#if WITH_EOS_SDK
	TUniquePtr<FEOSSDKManager> SDKManager;
#endif
	TArray<FString> SuppressedLogStrings_Log;
	TArray<FString> SuppressedLogCategories_Log;
	TArray<FString> SuppressedLogStrings_VeryVerbose;
	TArray<FString> SuppressedLogCategories_VeryVerbose;
	FDelegateHandle OnPostForkDelegateHandle;
};