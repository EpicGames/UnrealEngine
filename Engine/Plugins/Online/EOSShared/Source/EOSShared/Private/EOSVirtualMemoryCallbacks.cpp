// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVirtualMemoryCallbacks.h"

#if WITH_EOS_SDK

LLM_DEFINE_TAG(KITT);

#include "HAL/PlatformMemory.h"

namespace EOSVirtualMemoryCallbacks
{

void* EOS_MEMORY_CALL Reserve(size_t SizeInBytes, EOS_EVM_AccessType AccessType, void** OutContextData)
{
	LLM_PLATFORM_SCOPE_BYTAG(KITT);
	FPlatformMemory::FPlatformVirtualMemoryBlock* Block = new FPlatformMemory::FPlatformVirtualMemoryBlock(FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(SizeInBytes));
	*OutContextData = Block;
	return Block->GetVirtualPointer();
}

EOS_Bool EOS_MEMORY_CALL Release(void* Address, size_t SizeInBytes, void* ContextData)
{
	LLM_PLATFORM_SCOPE_BYTAG(KITT);
	FPlatformMemory::FPlatformVirtualMemoryBlock* Block = static_cast<FPlatformMemory::FPlatformVirtualMemoryBlock*>(ContextData);
	Block->FreeVirtual();
	delete Block;
	return true;
}

EOS_Bool EOS_MEMORY_CALL Commit(void* Address, size_t SizeInBytes, void* ContextData)
{
	LLM_PLATFORM_SCOPE_BYTAG(KITT);
	FPlatformMemory::FPlatformVirtualMemoryBlock* Block = static_cast<FPlatformMemory::FPlatformVirtualMemoryBlock*>(ContextData);
	Block->CommitByPtr(Address, SizeInBytes);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, static_cast<int64>(SizeInBytes)));
	return true;
}

EOS_Bool EOS_MEMORY_CALL Decommit(void* Address, size_t SizeInBytes, void* ContextData)
{
	LLM_PLATFORM_SCOPE_BYTAG(KITT);
	FPlatformMemory::FPlatformVirtualMemoryBlock* Block = static_cast<FPlatformMemory::FPlatformVirtualMemoryBlock*>(ContextData);
	Block->DecommitByPtr(Address, SizeInBytes);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Platform, -static_cast<int64>(SizeInBytes)));
	return true;
}

} // namespace EOSVirtualMemoryCallbacks

#endif // WITH_EOS_SDK