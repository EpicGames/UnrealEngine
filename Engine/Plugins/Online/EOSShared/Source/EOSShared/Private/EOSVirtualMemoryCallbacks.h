// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "HAL/LowLevelMemTracker.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_base.h"

enum class EOS_EVM_AccessType : int32_t;

LLM_DECLARE_TAG_API(KITT, EOSSHARED_API);

// Generic VMem callback implementations used by most platforms requiring VMem callbacks
namespace EOSVirtualMemoryCallbacks
{
	void* EOS_MEMORY_CALL Reserve(size_t SizeInBytes, EOS_EVM_AccessType AccessType, void** OutContextData);
	EOS_Bool EOS_MEMORY_CALL Release(void* Address, size_t SizeInBytes, void* ContextData);
	EOS_Bool EOS_MEMORY_CALL Commit(void* Address, size_t SizeInBytes, void* ContextData);
	EOS_Bool EOS_MEMORY_CALL Decommit(void* Address, size_t SizeInBytes, void* ContextData);
}

#endif // WITH_EOS_SDK