// Copyright Epic Games, Inc. All Rights Reserved.
#include "IOSEOSSDKManager.h"

#if WITH_EOS_SDK

#include "IOSAppDelegate.h"
#include "Misc/CoreDelegates.h"

static void OnUrlOpened(UIApplication* application, NSURL* url, NSString* sourceApplication, id annotation)
{
	// TODO: This is based on a prototype fix on EOS SDK. Once the fix is properly submitted to EOS SDK we should update it
	[[NSNotificationCenter defaultCenter] postNotificationName:@"EOSSDKAuthCallbackNotification" object:nil userInfo: @{@"EOSSDKAuthCallbackURLKey" : url}];
}

FIOSEOSSDKManager::FIOSEOSSDKManager()
{
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FIOSEOSSDKManager::OnApplicationForegroundChanged, false);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FIOSEOSSDKManager::OnApplicationForegroundChanged, true);
	FIOSCoreDelegates::OnOpenURL.AddStatic(&OnUrlOpened);
}

FIOSEOSSDKManager::~FIOSEOSSDKManager()
{	
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
}

FString FIOSEOSSDKManager::GetCacheDirBase() const
{
	NSString* BundleIdentifier = [[NSBundle mainBundle]bundleIdentifier];
	NSString* CacheDirectory = NSTemporaryDirectory(); // Potentially use NSCachesDirectory
	CacheDirectory = [CacheDirectory stringByAppendingPathComponent : BundleIdentifier];

	const char* CStrCacheDirectory = [CacheDirectory UTF8String];
	return FString(UTF8_TO_TCHAR(CStrCacheDirectory));
};

void FIOSEOSSDKManager::OnApplicationForegroundChanged(bool bWillBeOnBackground)
{
	EOS_EApplicationStatus NewStatus = bWillBeOnBackground ? EOS_EApplicationStatus::EOS_AS_BackgroundSuspended : EOS_EApplicationStatus::EOS_AS_Foreground;
	OnApplicationStatusChanged(NewStatus);
	if (bWillBeOnBackground)
	{
		Tick(0.f);
	}
}

#endif // WITH_EOS_SDK
