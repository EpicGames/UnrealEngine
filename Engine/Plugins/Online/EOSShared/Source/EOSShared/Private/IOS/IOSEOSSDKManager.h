// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSSDKManager.h"

class FIOSEOSSDKManager : public FEOSSDKManager
{
public:
	FIOSEOSSDKManager();
	~FIOSEOSSDKManager();

	virtual FString GetCacheDirBase() const override;
private:
	void OnApplicationForegroundChanged(bool bWillBeOnBackground);
};

using FPlatformEOSSDKManager = FIOSEOSSDKManager;

#endif // WITH_EOS_SDK
