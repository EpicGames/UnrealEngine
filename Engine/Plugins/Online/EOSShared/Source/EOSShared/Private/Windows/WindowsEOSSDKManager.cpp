// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EOS_SDK

#include "WindowsEOSSDKManager.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "EOSShared.h"
#include "Windows/eos_Windows.h"
#include "Modules/ModuleManager.h"

#if UE_WITH_EOS_STEAM_INTEGRATION
#include "ISteamSharedModule.h"
#endif

FWindowsEOSSDKManager::FWindowsEOSSDKManager()
{
#if UE_WITH_EOS_STEAM_INTEGRATION
	PlatformSteamOptions.ApiVersion = 3; //EOS_INTEGRATEDPLATFORM_STEAM_OPTIONS_API_LATEST;
	PlatformSteamOptions.OverrideLibraryPath = nullptr;
	PlatformSteamOptions.SteamMajorVersion = 1;
	PlatformSteamOptions.SteamMinorVersion = 57;
	PlatformSteamOptions.SteamApiInterfaceVersionsArray = nullptr;
	PlatformSteamOptions.SteamApiInterfaceVersionsArrayBytes = 0;

	UE_EOS_CHECK_API_MISMATCH(EOS_INTEGRATEDPLATFORM_STEAM_OPTIONS_API_LATEST, 3);

	GConfig->GetBool(TEXT("EOSSDK"), TEXT("bEnablePlatformIntegration"), bEnablePlatformIntegration, GEngineIni);
	
	if (bEnablePlatformIntegration)
	{
		ISteamSharedModule* SteamSharedModule = FModuleManager::LoadModulePtr<ISteamSharedModule>(TEXT("SteamShared"));
		
		if (!SteamSharedModule)
		{
			UE_LOGF(LogEOSShared, Warning, "SteamShared module not available, creation of platforms requiring Steam integration will fail, ensure SteamShared plugin is enabled");
		}
	}
#endif
}

FWindowsEOSSDKManager::~FWindowsEOSSDKManager()
{

}

EOS_EResult FWindowsEOSSDKManager::Initialize()
{
#if UE_WITH_EOS_STEAM_INTEGRATION
	if (bEnablePlatformIntegration)
	{
		// Register a shared pointer to Steam client handle. This is needed to ensure steam client handle is valid for EOS SDK to use.
		if (ISteamSharedModule* SteamSharedModule = FModuleManager::GetModulePtr<ISteamSharedModule>(TEXT("SteamShared")))
		{
			SteamAPIClientHandle = SteamSharedModule->ObtainSteamClientInstanceHandle();
		}
	}
#endif

	return Super::Initialize();
}

IEOSPlatformHandlePtr FWindowsEOSSDKManager::CreatePlatform(EOS_Platform_Options& PlatformOptions)
{
#if UE_WITH_EOS_STEAM_INTEGRATION
	// If native platform integration is enabled, and the Steam client isn't avaible, the EOS Platform should not be created
	if (bEnablePlatformIntegration && !SteamAPIClientHandle.IsValid())
	{
		UE_LOGF(LogEOSShared, Warning, "FWindowsEOSSDKManager::CreatePlatform failed. Steam integrated platform is enabled and steam initialization failed. EosPlatformHandle=nullptr");
		return nullptr;
	}
#endif 

	return Super::CreatePlatform(PlatformOptions);
}

IEOSPlatformHandlePtr FWindowsEOSSDKManager::CreatePlatform(const FEOSSDKPlatformConfig& PlatformConfig, EOS_Platform_Options& PlatformOptions)
{
	if (PlatformConfig.bWindowsEnableOverlayD3D9) PlatformOptions.Flags |= EOS_PF_WINDOWS_ENABLE_OVERLAY_D3D9;
	if (PlatformConfig.bWindowsEnableOverlayD3D10) PlatformOptions.Flags |= EOS_PF_WINDOWS_ENABLE_OVERLAY_D3D10;
	if (PlatformConfig.bWindowsEnableOverlayOpenGL) PlatformOptions.Flags |= EOS_PF_WINDOWS_ENABLE_OVERLAY_OPENGL;

	if (PlatformConfig.bEnableRTC)
	{
#if !PLATFORM_CPU_ARM_FAMILY
		static const FString XAudioPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/XAudio2_9/x64/xaudio2_9redist.dll")));
#else
		static const FString XAudioPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::Combine(FWindowsPlatformProcess::WindowsSystemDir(), TEXT("xaudio2_9.dll"))); // must use the system xaudio DLL as there's no arm64 redist version
#endif
		static const FTCHARToUTF8 Utf8XAudioPath(*XAudioPath);

		static EOS_Windows_RTCOptions WindowsRTCOptions = {};
		WindowsRTCOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_WINDOWS_RTCOPTIONS_API_LATEST, 1);
		WindowsRTCOptions.XAudio29DllPath = Utf8XAudioPath.Get();

		const_cast<EOS_Platform_RTCOptions*>(PlatformOptions.RTCOptions)->PlatformSpecificOptions = &WindowsRTCOptions;
	}

	return FEOSSDKManager::CreatePlatform(PlatformConfig, PlatformOptions);
}

FString FWindowsEOSSDKManager::GetCacheDirBase() const
{
	if (FPlatformMisc::IsCacheStorageAvailable())
	{
		// return folder path in AppData and not Documents to accomodate both user and system accounts
		return FWindowsPlatformProcess::UserSettingsDir();
	}
	else
	{
		return FString();
	}

}

const void* FWindowsEOSSDKManager::GetIntegratedPlatformOptions()
{
#if UE_WITH_EOS_STEAM_INTEGRATION
	return &PlatformSteamOptions;
#else
	return nullptr;
#endif
}

EOS_IntegratedPlatformType FWindowsEOSSDKManager::GetIntegratedPlatformType()
{
#if UE_WITH_EOS_STEAM_INTEGRATION
	return EOS_IPT_Steam;
#else
	return EOS_IPT_Unknown;
#endif
}

#endif // WITH_EOS_SDK