// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "EOSSDKManager.h"

class FWindowsEOSSDKManager : public FEOSSDKManager
{
public:
	using Super = FEOSSDKManager;

	FWindowsEOSSDKManager();
	virtual ~FWindowsEOSSDKManager();
	virtual EOS_EResult Initialize() override;
	virtual IEOSPlatformHandlePtr CreatePlatform(EOS_Platform_Options& PlatformOptions) override;
protected:
	virtual IEOSPlatformHandlePtr CreatePlatform(const FEOSSDKPlatformConfig& PlatformConfig, EOS_Platform_Options& PlatformOptions) override;

	virtual const void* GetIntegratedPlatformOptions() override;
	virtual EOS_IntegratedPlatformType GetIntegratedPlatformType() override;

#if UE_WITH_EOS_STEAM_INTEGRATION
	/** Steam Client API Handle */
	TSharedPtr<class FSteamClientInstanceHandler> SteamAPIClientHandle;

	EOS_IntegratedPlatform_Steam_Options PlatformSteamOptions;
#endif 

public: 
	virtual FString GetCacheDirBase() const override;
};

using FPlatformEOSSDKManager = FWindowsEOSSDKManager;

#endif // WITH_EOS_SDK