// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "HAL/PreprocessorHelpers.h"

#ifndef UE_WITH_EOS_SDK_APIVERSION_WARNINGS
#define UE_WITH_EOS_SDK_APIVERSION_WARNINGS 1
#endif

/**
 * Emits warnings when an SDK upgrade bumps the ApiVersion, to let you know something changed and you should at least check the changes.
 * To silence warnings, either bump the Actual value to match the new Expected once you have checked the API changes, or in the Engine ini hierarchy, set
 * 
 * [EOSShared]
 * bEnableApiVersionWarnings=false
 */
#if UE_WITH_EOS_SDK_APIVERSION_WARNINGS
#define UE_EOS_CHECK_API_MISMATCH(Actual, Expected) \
	struct UE_JOIN(FEosApiMismatchMsg_, __LINE__) { \
		[[deprecated(#Actual " updated from " #Expected " to " UE_STRINGIZE(Actual) ", behaviour/params may have changed")]] \
		static constexpr int condition(TStaticDeprecateExpression<true>) { return 1; } \
		static constexpr int condition(TStaticDeprecateExpression<false>) { return 1; } \
	}; \
	enum class UE_JOIN(EEosApiMismatchMsg_, __LINE__) { Value = UE_JOIN(FEosApiMismatchMsg_, __LINE__)::condition(TStaticDeprecateExpression<Actual != Expected>()) }
#else
#define UE_EOS_CHECK_API_MISMATCH(Actual, Expected)
#endif

#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_base.h"
#include "eos_common.h"
#include "eos_types.h"
#include "eos_version.h"

#ifndef WITH_EOS_RTC
#define WITH_EOS_RTC WITH_EOS_SDK && (EOS_MAJOR_VERSION >= 1 && EOS_MINOR_VERSION >= 13)
#endif

#ifndef WITH_EOS_P2P
#define WITH_EOS_P2P WITH_EOS_SDK
#endif

#define EOS_ENUM_FORWARD_DECL(name) enum class name : int32_t;
EOS_ENUM_FORWARD_DECL(EOS_EApplicationStatus);
EOS_ENUM_FORWARD_DECL(EOS_EAuthScopeFlags);
EOS_ENUM_FORWARD_DECL(EOS_EAuthTokenType);
EOS_ENUM_FORWARD_DECL(EOS_EDesktopCrossplayStatus);
EOS_ENUM_FORWARD_DECL(EOS_EFriendsStatus);
EOS_ENUM_FORWARD_DECL(EOS_ELoginCredentialType);
EOS_ENUM_FORWARD_DECL(EOS_ENetworkStatus);
EOS_ENUM_FORWARD_DECL(EOS_EPacketReliability);
EOS_ENUM_FORWARD_DECL(EOS_ERTCAudioInputStatus);
EOS_ENUM_FORWARD_DECL(EOS_Presence_EStatus);
EOS_ENUM_FORWARD_DECL(EOS_UI_EInputStateButtonFlags);
#undef EOS_ENUM_FORWARD_DECL

#define EOS_STRUCT_FORWARD_DECL(name) extern "C" typedef struct _tag ## name name;
EOS_STRUCT_FORWARD_DECL(EOS_UserInfo);
EOS_STRUCT_FORWARD_DECL(EOS_UserInfo_BestDisplayName);
EOS_STRUCT_FORWARD_DECL(EOS_RTC_Option);
#undef EOS_STRUCT_FORWARD_DECL

extern "C" typedef uint32_t EOS_OnlinePlatformType;

DECLARE_LOG_CATEGORY_EXTERN(LogEOSSDK, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogEOSShared, Log, All);

EOSSHARED_API FString LexToString(const EOS_EResult EosResult);
EOSSHARED_API const char* LexToUtf8String(const EOS_EResult EosResult);

EOSSHARED_API FString LexToString(const EOS_ProductUserId UserId);
EOSSHARED_API void LexFromString(EOS_ProductUserId& UserId, const TCHAR* String);
inline EOS_ProductUserId EOSProductUserIdFromString(const TCHAR* String)
{
	EOS_ProductUserId UserId;
	LexFromString(UserId, String);
	return UserId;
}

EOSSHARED_API FString LexToString(const EOS_EpicAccountId AccountId);
EOSSHARED_API void LexFromString(EOS_EpicAccountId& AccountId, const TCHAR* String);

EOSSHARED_API const TCHAR* LexToString(const EOS_EApplicationStatus ApplicationStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_EAuthTokenType AuthTokenType);
EOSSHARED_API const TCHAR* LexToString(const EOS_EDesktopCrossplayStatus DesktopCrossplayStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_EExternalAccountType ExternalAccountType);
EOSSHARED_API const TCHAR* LexToString(const EOS_EFriendsStatus FriendStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_ELoginStatus LoginStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_ENetworkStatus NetworkStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_ERTCAudioInputStatus Status);
EOSSHARED_API const TCHAR* LexToString(const EOS_Presence_EStatus PresenceStatus);
EOSSHARED_API const TCHAR* LexToString(const EOS_EExternalCredentialType ExternalCredentialType);

EOSSHARED_API bool LexFromString(EOS_EAuthScopeFlags& OutEnum, const FStringView& InString);
EOSSHARED_API bool LexFromString(EOS_EExternalAccountType& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_EIntegratedPlatformManagementFlags& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_ELoginCredentialType& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_EPacketReliability& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_ERTCBackgroundMode& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_UI_EInputStateButtonFlags& OutEnum, const TCHAR* InString);
EOSSHARED_API bool LexFromString(EOS_EExternalCredentialType& OutEnum, const TCHAR* InString);

EOSSHARED_API EOS_OnlinePlatformType EOSOnlinePlatformTypeFromString(const FStringView& InString);

/** Extracts the display name FString from a EOS_UserInfo_BestDisplayName using the following logic: Nickname > DisplayNameSanitized > DisplayName */
EOSSHARED_API FString GetBestDisplayNameStr(const EOS_UserInfo_BestDisplayName& BestDisplayName);
/** Extracts the display name FString from a EOS_UserInfo using the following logic: Nickname > DisplayNameSanitized > DisplayName */
EOSSHARED_API FString GetUserInfoDisplayNameStr(const EOS_UserInfo& UserInfo);

EOSSHARED_API FString LexToString(const EOS_RTC_Option& Option);

#endif // WITH_EOS_SDK