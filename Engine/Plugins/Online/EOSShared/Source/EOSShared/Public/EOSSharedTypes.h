// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreGlobals.h"
#include "EOSShared.h"
#include "Templates/SharedPointer.h"

#if WITH_EOS_SDK
#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_common.h"
#endif

// Expect URLs to look like "EOS:PUID:SocketName:Channel"
#define EOS_CONNECTION_URL_PREFIX TEXT("EOS")
#define EOS_URL_SEPARATOR TEXT(":")

/** Used to store a pointer to the EOS callback object without knowing type */
class FCallbackBase
{
public:
	virtual ~FCallbackBase() = default;
};

#if WITH_EOS_SDK

/** Wrapper for EOS notification callbacks, ensures calling object is still alive before calling passed lambda. */
template<typename CallbackFuncType, typename CallbackParamType, typename OwningType, typename CallbackReturnType = void, typename... CallbackExtraParams>
class TEOSGlobalCallback :
	public FCallbackBase
{
public:
	TFunction<CallbackReturnType(const CallbackParamType*, CallbackExtraParams... ExtraParams)> CallbackLambda;

	TEOSGlobalCallback(TWeakPtr<OwningType> InOwner)
		: FCallbackBase()
		, Owner(InOwner)
	{
	}
	virtual ~TEOSGlobalCallback() = default;

	CallbackFuncType GetCallbackPtr()
	{
		return &CallbackImpl;
	}

	/** Is this callback intended for the game thread */
	bool bIsGameThreadCallback = true;

private:
	/** The object that needs to be checked for lifetime before calling the callback */
	TWeakPtr<OwningType> Owner;

	static CallbackReturnType EOS_CALL CallbackImpl(const CallbackParamType* Data, CallbackExtraParams... ExtraParams)
	{
		TEOSGlobalCallback* CallbackThis = (TEOSGlobalCallback*)Data->ClientData;
		check(CallbackThis);

		if (CallbackThis->bIsGameThreadCallback)
		{
			check(IsInGameThread());
		}

		TSharedPtr <OwningType> Pinned = CallbackThis->Owner.Pin();
		if (Pinned.IsValid())
		{
			check(CallbackThis->CallbackLambda);

			if constexpr (std::is_void<CallbackReturnType>::value)
			{
				CallbackThis->CallbackLambda(Data, ExtraParams...);
			}
			else
			{
				return CallbackThis->CallbackLambda(Data, ExtraParams...);
			}
		}
		
		if constexpr (!std::is_void<CallbackReturnType>::value)
		{
			// we need to return _something_ to compile.
			return CallbackReturnType{};
		}
	}
};

/** Wrapper for EOS async API callbacks, ensures the caller is still alive before calling passed lambda, then deletes itself. */
template<typename CallbackFuncType, typename CallbackType, typename OwningType>
class TEOSCallback :
	public FCallbackBase
{
public:
	TFunction<void(const CallbackType*)> CallbackLambda;

	TEOSCallback(TWeakPtr<OwningType> InOwner)
		: FCallbackBase()
		, Owner(InOwner)
	{
	}
	TEOSCallback(TWeakPtr<const OwningType> InOwner)
		: FCallbackBase()
		, Owner(InOwner)
	{
	}
	virtual ~TEOSCallback() = default;

	CallbackFuncType GetCallbackPtr()
	{
		return &CallbackImpl;
	}

protected:
	/** The object that needs to be checked for lifetime before calling the callback */
	TWeakPtr<const OwningType> Owner;

private:
	static void EOS_CALL CallbackImpl(const CallbackType* Data)
	{
		if (EOS_EResult_IsOperationComplete(Data->ResultCode) == EOS_FALSE)
		{
			// Ignore
			return;
		}
		check(IsInGameThread());

		TEOSCallback* CallbackThis = (TEOSCallback*)Data->ClientData;
		check(CallbackThis);

		if (CallbackThis->Owner.IsValid())
		{
			check(CallbackThis->CallbackLambda);
			CallbackThis->CallbackLambda(Data);
		}
		delete CallbackThis;
	}
};

namespace UE::Online {

namespace Private {

/** Class to handle all callbacks generically using a future to forward callback results */
template<typename CallbackType>
class TEOSCallback
{
	using CallbackFuncType = void (EOS_CALL*)(const CallbackType*);
public:
	TEOSCallback() = default;
	TEOSCallback(TPromise<const CallbackType*>&& InPromise) : Promise(MoveTemp(InPromise)) {}
	virtual ~TEOSCallback() = default;

	operator CallbackFuncType()
	{
		return &CallbackImpl;
	}

	TFuture<const CallbackType*> GetFuture() { return Promise.GetFuture(); }

private:
	TPromise<const CallbackType*> Promise;

	static void EOS_CALL CallbackImpl(const CallbackType* Data)
	{
		if (EOS_EResult_IsOperationComplete(Data->ResultCode) == EOS_FALSE)
		{
			// Ignore
			return;
		}
		check(IsInGameThread());

		TEOSCallback* CallbackThis = reinterpret_cast<TEOSCallback*>(Data->ClientData);
		check(CallbackThis);

		CallbackThis->Promise.EmplaceValue(Data);

		delete CallbackThis;
	}
};

template<typename Function> struct FEOSAsyncCallbackTraits;
template<typename TCallbackData>
struct FEOSAsyncCallbackTraits<void(*)(const TCallbackData*)>
{
	typedef TCallbackData CallbackDataType;
};

template<typename Function> struct FEOSAsyncTraits;
template<typename TEOSHandle, typename TEOSParameters, typename TEOSCallback>
struct FEOSAsyncTraits<void(*)(TEOSHandle, const TEOSParameters*, void*, const TEOSCallback)>
{
	typedef TEOSCallback CallbackType;
	typedef typename FEOSAsyncCallbackTraits<TEOSCallback>::CallbackDataType CallbackDataType;
};

/* Private */ }

/**
 * This method requires a Promise that will be fulfilled by the EOS callback.
 * This is primarily intended to be used with continuations that take a TPromise. e.g.:
 *
 *		InAsyncOp.Then([this, LoginOptions, Credentials](TOnlineAsyncOp<FAuthLogin>& InAsyncOp, TPromise<const EOS_Auth_LoginCallbackInfo*>&& Promise) mutable
 *		{
 *			LoginOptions.Credentials = &Credentials;
 *			EOS_Async(EOS_Auth_Login, AuthHandle, LoginOptions, MoveTemp(Promise));
 *		})
 * 
 * Note that when a continuation takes a Promise parameter, the Future for that Promise is already bound
 * to the next continuation in the Op chain. Therefore, as soon as the EOS_Auth_Login callback is called,
 * the promise will be fulfilled, and the next continuation will run immediately. This ensures that even
 * if the SDK calls the callback immediately (i.e. before EOS_Async returns), the continuation is already
 * bound, and we can safely consume the CallbackInfo. Note that the CallbackInfo is only valid for the
 * duration of the callback, so it is not safe to bind a continuation _after_ calling the EOS method.
 */
// Warning: This signature will crash if the async method returns immediately.
// The callback object is deleted inside the callback before EOS_Async returns the future back to the user.
template<typename TEOSResult, typename TEOSHandle, typename TEOSParameters, typename TEOSFn>
decltype(auto) EOS_Async(TEOSFn EOSFn, TEOSHandle EOSHandle, TEOSParameters Parameters, TPromise<const TEOSResult*>&& Promise)
{
	Private::TEOSCallback<TEOSResult>* Callback = new Private::TEOSCallback<TEOSResult>(Forward< TPromise<const TEOSResult*>>(Promise));
	return EOSFn(EOSHandle, &Parameters, Callback, *Callback);
}

/**
 * This method requires a Callback (lambda / function pointer) which will be called by the EOS callback
 * with the CallbackInfo.
 */
template<typename TEOSHandle, typename TEOSParameters, typename TEOSFn, typename TCallback>
decltype(auto) EOS_Async(TEOSFn EOSFn, TEOSHandle EOSHandle, const TEOSParameters& Parameters, TCallback&& Callback)
{
	using CallbackDataType = typename Private::FEOSAsyncTraits<TEOSFn>::CallbackDataType;
	TUniquePtr<TCallback> CallbackObj = MakeUnique<TCallback>(Forward<TCallback>(Callback));
	return EOSFn(EOSHandle, &Parameters, CallbackObj.Release(),
	static_cast<void(EOS_CALL*)(const CallbackDataType* Data)>([](const CallbackDataType* Data)
	{
		if (EOS_EResult_IsOperationComplete(Data->ResultCode))
		{
			check(IsInGameThread());
			TUniquePtr<TCallback> CallbackObj(reinterpret_cast<TCallback*>(Data->ClientData));
			check(CallbackObj);
			(*CallbackObj)(Data);
		}
	}));
}

class FEOSEventRegistration
{
public:
	virtual ~FEOSEventRegistration() = default;
};
typedef TUniquePtr<FEOSEventRegistration> FEOSEventRegistrationPtr;

using EOSEventRegistration UE_DEPRECATED(5.6, "EOSEventRegistration renamed to FEOSEventRegistration") = FEOSEventRegistration;
using EOSEventRegistrationPtr UE_DEPRECATED(5.6, "EOSEventRegistrationPtr renamed to FEOSEventRegistrationPtr") = FEOSEventRegistrationPtr;

/** 
* EOS event registration utility for binding an EOS notifier registration to a RAII object which handles
* unregistering when it exits scope. Intended to be used from a TOnlineComponent class.
* 
* Example:
*	EOSEventRegistrationPtr OnLobbyUpdatedEOSEventRegistration = EOS_RegisterComponentEventHandler(
*		this,
*		LobbyHandle,
*		EOS_LOBBY_ADDNOTIFYLOBBYUPDATERECEIVED_API_LATEST,
*		&EOS_Lobby_AddNotifyLobbyUpdateReceived,
*		&EOS_Lobby_RemoveNotifyLobbyUpdateReceived,
*		&FLobbiesEOS::HandleLobbyUpdated);
**/
template <
	typename ComponentHandlerClass,
	typename EOSHandle,
	typename EOSNotfyRegisterFunction,
	typename EOSNotfyUnregisterFunction,
	typename ComponentHandlerFunction>
FEOSEventRegistrationPtr EOS_RegisterComponentEventHandler(
	ComponentHandlerClass* HandlerClass,
	EOSHandle ClientHandle,
	int32_t ApiVersion,
	EOSNotfyRegisterFunction NotfyRegisterFunction,
	EOSNotfyUnregisterFunction NotfyUnregisterFunction,
	ComponentHandlerFunction HandlerFunction);

namespace Private {

template<typename Function> struct TEOSCallbackTraitsBase;

template<typename Component, typename EventData>
struct TEOSCallbackTraitsBase<void (Component::*)(const EventData*)>
{
	typedef Component ComponentType;
	typedef EventData EventDataType;
};

template<typename Function>
struct TEOSCallbackTraits : public TEOSCallbackTraitsBase<typename std::remove_reference<Function>::type>
{
};

template<typename Function> struct TEOSNotifyRegisterTraits;

template<typename EOSHandle, typename Options, typename ClientData, typename NotificationFn, typename NotificationId>
struct TEOSNotifyRegisterTraits<NotificationId (*)(EOSHandle, const Options*, ClientData*, NotificationFn)>
{
	typedef Options OptionsType;
	typedef NotificationId NotificationIdType;
};

template <
	typename ComponentHandlerClass,
	typename EOSHandle,
	typename EOSNotfyRegisterFunction,
	typename EOSNotfyUnregisterFunction,
	typename ComponentHandlerFunction>
class FEOSEventRegistrationImpl : public FEOSEventRegistration
{
public:
	typedef FEOSEventRegistrationImpl<
		ComponentHandlerClass,
		EOSHandle,
		EOSNotfyRegisterFunction,
		EOSNotfyUnregisterFunction,
		ComponentHandlerFunction> ThisClass;

	FEOSEventRegistrationImpl(
		ComponentHandlerClass* HandlerClass,
		EOSHandle ClientHandle,
		int32_t ApiVersion,
		EOSNotfyRegisterFunction NotfyRegisterFunction,
		EOSNotfyUnregisterFunction NotfyUnregisterFunction,
		ComponentHandlerFunction HandlerFunction)
		: HandlerClass(HandlerClass)
		, ClientHandle(ClientHandle)
		, NotfyUnregisterFunction(NotfyUnregisterFunction)
		, HandlerFunction(HandlerFunction)
	{
		using EventDataType = typename TEOSCallbackTraits<ComponentHandlerFunction>::EventDataType;
		typename TEOSNotifyRegisterTraits<EOSNotfyRegisterFunction>::OptionsType Options = { };
		Options.ApiVersion = ApiVersion;
		NotificationId = NotfyRegisterFunction(ClientHandle, &Options, this,
		static_cast<void(EOS_CALL*)(const EventDataType* Data)>([](const EventDataType* Data)
		{
			ThisClass* This = reinterpret_cast<ThisClass*>(Data->ClientData);
			(This->HandlerClass->*This->HandlerFunction)(Data);
		}));
	}

	virtual ~FEOSEventRegistrationImpl()
	{
		NotfyUnregisterFunction(ClientHandle, NotificationId);
	}

private:
	FEOSEventRegistrationImpl() = delete;
	FEOSEventRegistrationImpl(const FEOSEventRegistrationImpl&) = delete;
	FEOSEventRegistrationImpl& operator=(const FEOSEventRegistrationImpl&) = delete;

	typedef typename TEOSNotifyRegisterTraits<EOSNotfyRegisterFunction>::NotificationIdType NotificationIdType;

	NotificationIdType NotificationId;
	ComponentHandlerClass* HandlerClass;
	EOSHandle ClientHandle;
	EOSNotfyUnregisterFunction NotfyUnregisterFunction;
	ComponentHandlerFunction HandlerFunction;
};

/* Private */ }

template <
	typename ComponentHandlerClass,
	typename EOSHandle,
	typename EOSNotfyRegisterFunction,
	typename EOSNotfyUnregisterFunction,
	typename ComponentHandlerFunction>
FEOSEventRegistrationPtr EOS_RegisterComponentEventHandler(
	ComponentHandlerClass* HandlerClass,
	EOSHandle ClientHandle,
	int32_t ApiVersion,
	EOSNotfyRegisterFunction NotfyRegisterFunction,
	EOSNotfyUnregisterFunction NotfyUnregisterFunction,
	ComponentHandlerFunction HandlerFunction)
{
	return MakeUnique<
		Private::FEOSEventRegistrationImpl<
			ComponentHandlerClass,
			EOSHandle,
			EOSNotfyRegisterFunction,
			EOSNotfyUnregisterFunction,
			ComponentHandlerFunction>>(
			HandlerClass,
			ClientHandle,
			ApiVersion,
			NotfyRegisterFunction,
			NotfyUnregisterFunction,
			HandlerFunction);
}

/* UE::Online */ }

#endif
