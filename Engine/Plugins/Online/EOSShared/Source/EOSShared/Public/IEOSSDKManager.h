// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EOS_SDK

#include "Features/IModularFeatures.h"
#include "Templates/UniquePtr.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_init.h"
#include "eos_types.h"

struct FEOSSDKPlatformConfig
{
	FString Name;
	FString ProductId;
	FString SandboxId;
	FString ClientId;
	FString ClientSecret;
	FString EncryptionKey;
	FString RelyingPartyURI;
	FString OverrideCountryCode;
	FString OverrideLocaleCode;
	FString DeploymentId;
	FString CacheDirectory;
	EOS_ERTCBackgroundMode RTCBackgroundMode = EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive;
	bool bIsServer = false;
	bool bLoadingInEditor = false;
	bool bDisableOverlay = false;
	bool bDisableSocialOverlay = false;
	bool bWindowsEnableOverlayD3D9 = false;
	bool bWindowsEnableOverlayD3D10 = false;
	bool bWindowsEnableOverlayOpenGL = false;
	bool bEnableRTC = true;
	int32 TickBudgetInMilliseconds = 1;
	TArray<FString> OptionalConfig;
};

/**
* Allows temporary RAII style overriding of the config driven tick rate, for scenarios where you want the SDK
* To tick as fast as possible, i.e. overlay visible, or time critical operations
*/
class IEOSFastTickLock
{
public:
	virtual ~IEOSFastTickLock() = default;
};

class IEOSPlatformHandle
{
public:
	IEOSPlatformHandle(EOS_HPlatform InPlatformHandle)
		: PlatformHandle(InPlatformHandle)
	{}

	virtual ~IEOSPlatformHandle() = default;

	virtual void Tick() = 0;
	virtual TSharedRef<IEOSFastTickLock> GetFastTickLock() = 0;

	operator EOS_HPlatform() const { return PlatformHandle; }

	virtual FString GetConfigName() const = 0;
	virtual FString GetOverrideCountryCode() const = 0;
	virtual FString GetOverrideLocaleCode() const = 0;

	virtual void LogInfo(int32 Indent = 0) const = 0;
	virtual void LogAuthInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const = 0;
	virtual void LogUserInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const = 0;
	virtual void LogPresenceInfo(const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const = 0;
	virtual void LogFriendsInfo(const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const = 0;
	virtual void LogConnectInfo(const EOS_ProductUserId LoggedInAccount, int32 Indent = 0) const = 0;

protected:
	EOS_HPlatform PlatformHandle;
};

using IEOSPlatformHandlePtr = TSharedPtr<IEOSPlatformHandle, ESPMode::ThreadSafe>;
using IEOSPlatformHandleWeakPtr = TWeakPtr<IEOSPlatformHandle, ESPMode::ThreadSafe>;

// This callback lets you modify the options struct
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPreInitializeSDK, EOS_InitializeOptions& Options);
// This callback lets you modify or replace the options struct
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPreInitializeSDK2, EOS_InitializeOptions*& InOutOptions);
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPostInitializeSDK, EOS_EResult Result);
DECLARE_MULTICAST_DELEGATE_TwoParams(FEOSSDKManagerOnDefaultPlatformConfigNameChanged, const FString& NewName, const FString& OldName);
DECLARE_MULTICAST_DELEGATE_TwoParams(FEOSSDKManagerOnPreCreateNamedPlatform, const FEOSSDKPlatformConfig& Config, EOS_Platform_Options& Options);
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPreCreatePlatform, EOS_Platform_Options& Options);
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPlatformCreated, const IEOSPlatformHandlePtr& PlatformHandle);
DECLARE_MULTICAST_DELEGATE_OneParam(FEOSSDKManagerOnPreReleasePlatform, const EOS_HPlatform& PlatformHandle);
DECLARE_MULTICAST_DELEGATE_TwoParams(FEOSSDKManagerOnNetworkStatusChanged, const EOS_ENetworkStatus OldNetworkStatus, const EOS_ENetworkStatus NewNetworkStatus);
DECLARE_DELEGATE_RetVal(FString, FEOSSDKManagerOnRequestRuntimeLibraryName);

class IEOSSDKManager : public IModularFeature
{
public:
	static IEOSSDKManager* Get()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
		{
			return &IModularFeatures::Get().GetModularFeature<IEOSSDKManager>(GetModularFeatureName());
		}
		return nullptr;
	}

	static FName GetModularFeatureName()
	{
		static const FName FeatureName = TEXT("EOSSDKManager");
		return FeatureName;
	}

	virtual ~IEOSSDKManager() = default;

	virtual bool IsInitialized() const = 0;

	virtual const FEOSSDKPlatformConfig* GetPlatformConfig(const FString& PlatformConfigName, bool bLoadIfMissing = false) = 0;
	virtual bool AddPlatformConfig(const FEOSSDKPlatformConfig& PlatformConfig, bool bOverwriteExistingConfig = false) = 0;
	virtual const FString& GetDefaultPlatformConfigName() = 0;
	virtual void SetDefaultPlatformConfigName(const FString& PlatformConfigName) = 0;

	/**
	 * Create a platform handle for a platform config name. Config is loaded from .ini files if it was not added with AddPlatformConfig.
	 * If a platform handle already exists for the config name, this will return a shared pointer to that handle and not create a new one.
	 */
	virtual IEOSPlatformHandlePtr CreatePlatform(const FString& PlatformConfigName, FName InstanceName = NAME_None) = 0;

	/** Create a platform handle using EOSSDK options directly. */
	virtual IEOSPlatformHandlePtr CreatePlatform(EOS_Platform_Options& PlatformOptions) = 0;

	/** Retrieves the array of platform handles for all active platforms */
	virtual TArray<IEOSPlatformHandlePtr> GetActivePlatforms() = 0;

	virtual FString GetSDKVersion() const = 0;
	virtual FString GetProductName() const = 0;
	virtual FString GetProductVersion() const = 0;
	virtual FString GetCacheDirBase() const = 0;
	virtual FString GetOverrideCountryCode(const EOS_HPlatform Platform) const = 0;
	virtual FString GetOverrideLocaleCode(const EOS_HPlatform Platform) const = 0;

	virtual void LogInfo(int32 Indent = 0) const = 0;
	virtual void LogPlatformInfo(const EOS_HPlatform Platform, int32 Indent = 0) const = 0;
	virtual void LogAuthInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const = 0;
	virtual void LogUserInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const = 0;
	virtual void LogPresenceInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, const EOS_EpicAccountId TargetAccount, int32 Indent = 0) const = 0;
	virtual void LogFriendsInfo(const EOS_HPlatform Platform, const EOS_EpicAccountId LoggedInAccount, int32 Indent = 0) const = 0;
	virtual void LogConnectInfo(const EOS_HPlatform Platform, const EOS_ProductUserId LoggedInAccount, int32 Indent = 0) const = 0;

	/** Assign ownership of a callback object, to be free'd after EOS_Shutdown */
	virtual void AddCallbackObject(TUniquePtr<class FCallbackBase> CallbackObj) = 0;

	virtual TSharedRef<IEOSFastTickLock> GetFastTickLock() = 0;

	FEOSSDKManagerOnPreInitializeSDK OnPreInitializeSDK;
	FEOSSDKManagerOnPreInitializeSDK2 OnPreInitializeSDK2;
	FEOSSDKManagerOnPostInitializeSDK OnPostInitializeSDK;
	FEOSSDKManagerOnDefaultPlatformConfigNameChanged OnDefaultPlatformConfigNameChanged;
	FEOSSDKManagerOnPreCreateNamedPlatform OnPreCreateNamedPlatform;
	FEOSSDKManagerOnPreCreatePlatform OnPreCreatePlatform;
	FEOSSDKManagerOnPlatformCreated OnPlatformCreated;
	FEOSSDKManagerOnPreReleasePlatform OnPreReleasePlatform;
	FEOSSDKManagerOnRequestRuntimeLibraryName OnRequestRuntimeLibraryName;
	FEOSSDKManagerOnNetworkStatusChanged OnNetworkStatusChanged;
};

#endif // WITH_EOS_SDK